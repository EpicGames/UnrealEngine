{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SVG Importer",
	"Description": "Importing and handling SVG files",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SVGImporter",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SVGImporterEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryScripting",
			"Enabled":  true
		},
		{
			"Name": "GeometryMask",
			"Enabled": true
		}
	]
}
