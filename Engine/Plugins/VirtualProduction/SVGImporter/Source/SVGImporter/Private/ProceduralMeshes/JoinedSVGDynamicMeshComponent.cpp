// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshes/JoinedSVGDynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Interfaces/IPluginManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "SVGActor.h"
#include "SVGImporterUtils.h"
#include "UObject/ConstructorHelpers.h"

UJoinedSVGDynamicMeshComponent::UJoinedSVGDynamicMeshComponent()
{
	bSVGIsUnlit = true;
	Coloring = EJoinedSVGMeshColoring::SeparateColors;

	SVGStoredMesh = CreateDefaultSubobject<UDynamicMesh>("SVGStoredMesh");

	LoadResources();
}

void UJoinedSVGDynamicMeshComponent::Initialize(const FJoinedSVGMeshParameters& InJoinedMeshParameters)
{
	TArray<FLinearColor> Colors;

	ShapeParametersList = InJoinedMeshParameters.ShapesParameters;
	for (const FSVGShapeParameters& ShapeParameters : ShapeParametersList)
	{
		Colors.Add(ShapeParameters.Color);
	}

	MainColor = FSVGImporterUtils::GetAverageColor(Colors);

	SetSVGIsUnlit(InJoinedMeshParameters.bIsUnlit);
}

void UJoinedSVGDynamicMeshComponent::LoadResources()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("SVGImporter"));
	if (ensure(Plugin.IsValid()))
	{
		const FString ContentRoot = Plugin->GetMountedAssetPath();
		const FString SVGMaterialsPath = FPaths::Combine(TEXT("Material'"), ContentRoot, TEXT("Materials"));

		const FString SVGMainMaterialPath       = FPaths::Combine(SVGMaterialsPath, TEXT("SVGMainMaterial.SVGMainMaterial'"));
		const FString SVGMainMaterialPath_Unlit = FPaths::Combine(SVGMaterialsPath, TEXT("SVGMainMaterial_Unlit.SVGMainMaterial_Unlit'"));

		static ConstructorHelpers::FObjectFinder<UMaterial> BasicTextMaterialFinder(*SVGMainMaterialPath);

		if (BasicTextMaterialFinder.Succeeded())
		{
			MeshMaterial_Lit = BasicTextMaterialFinder.Object;
		}

		static ConstructorHelpers::FObjectFinder<UMaterial> BasicTextMaterialUnlitFinder(*SVGMainMaterialPath_Unlit);

		if (BasicTextMaterialUnlitFinder.Succeeded())
		{
			MeshMaterial_Unlit = BasicTextMaterialUnlitFinder.Object;
		}
	}
}

void UJoinedSVGDynamicMeshComponent::StoreMaterialSetParameters()
{
	if (ShapesMaterials.IsEmpty())
	{
		return;
	}

	for (FSVGShapeParameters& ShapeParameters : ShapeParametersList)
	{
		const int32 MaterialID = ShapeParameters.MaterialID;

		if (!ShapesMaterials.IsValidIndex(MaterialID))
		{
			continue;
		}

		if (const TObjectPtr<UMaterialInstanceDynamic>& Material = ShapesMaterials[MaterialID])
		{
			FLinearColor Color;
			Material->GetVectorParameterValue(FMaterialParameterInfo(TEXT("Color")), Color);
			ShapeParameters.Color = Color;
		}
	}
}

void UJoinedSVGDynamicMeshComponent::StoreCurrentMesh()
{
	ProcessMesh([this](const FDynamicMesh3& SourceMesh)
	{
		SVGStoredMesh->EditMesh([&SourceMesh](FDynamicMesh3& EditMesh)
		{
			EditMesh = SourceMesh;
		});
	});

	StoreMaterialSetParameters();
}

void UJoinedSVGDynamicMeshComponent::LoadStoredMesh()
{
	if (!SVGStoredMesh->IsEmpty())
	{
		EditMesh([this](FDynamicMesh3& EditMesh)
		{
			SVGStoredMesh->ProcessMesh([&EditMesh](const FDynamicMesh3& SourceMesh)
			{
				EditMesh = SourceMesh;
			});
		});

		UpdateMaterials();
	}
	else
	{
		StoreCurrentMesh();
	}

	MarkSVGMeshUpdated();
}

void UJoinedSVGDynamicMeshComponent::PostLoad()
{
	Super::PostLoad();
	LoadStoredMesh();
}

#if WITH_EDITOR
void UJoinedSVGDynamicMeshComponent::PostEditImport()
{
	Super::PostEditImport();
	LoadStoredMesh();
}

void UJoinedSVGDynamicMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	static const FName IsSVGUnlitName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, bSVGIsUnlit);
	static const FName MeshParametersName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, ShapeParametersList);
	static const FName MainColorName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, MainColor);
	static const FName ColoringName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, Coloring);

	if (MemberName == IsSVGUnlitName)
	{
		RefreshSVGIsUnlit();
	}
	else if (MemberName == MeshParametersName || MemberName == MainColorName)
	{
		UpdateMaterials();
	}
	else if (MemberName == ColoringName)
	{
		UpdateMaterials(true);
	}
}
#endif

void UJoinedSVGDynamicMeshComponent::RefreshSVGIsUnlit()
{
	constexpr bool bRefreshMaterials = true;
	UpdateMaterials(bRefreshMaterials);
}

void UJoinedSVGDynamicMeshComponent::SetSVGIsUnlit(bool bInSVGIsUnlit)
{
	if (bSVGIsUnlit != bInSVGIsUnlit)
	{
		bSVGIsUnlit = bInSVGIsUnlit;
		RefreshSVGIsUnlit();
	}
}

void UJoinedSVGDynamicMeshComponent::SetMainColor(const FLinearColor& InColor)
{
	if (MainColor != InColor)
	{
		MainColor = InColor;
		UpdateMaterials();
	}
}

void UJoinedSVGDynamicMeshComponent::LoadMaterialSetParameters()
{
	for (const FSVGShapeParameters& ShapeParameters : ShapeParametersList)
	{
		const int32 MaterialID = ShapeParameters.MaterialID;

		if (!ShapesMaterials.IsValidIndex(MaterialID))
		{
			continue;
		}

		if (const TObjectPtr<UMaterialInstanceDynamic>& Material = ShapesMaterials[MaterialID])
		{
			FLinearColor ColorToApply = FLinearColor::Black;

			switch (Coloring)
			{
				case EJoinedSVGMeshColoring::SeparateColors:
					ColorToApply = ShapeParameters.Color;
					break;

				case EJoinedSVGMeshColoring::SingleColor:
					ColorToApply = MainColor;
					break;
			}

			Material->SetVectorParameterValue("Color", ColorToApply);
		}
	}
}

void UJoinedSVGDynamicMeshComponent::UpdateMaterials(bool bInRefreshInstances)
{
	const bool bMaterialsNeedsToRefresh = bInRefreshInstances || ShapesMaterials.IsEmpty();

	if (bMaterialsNeedsToRefresh)
	{
		UMaterial* CurrentMaterial = nullptr;
		if (bSVGIsUnlit)
		{
			if (MeshMaterial_Unlit)
			{
				CurrentMaterial = MeshMaterial_Unlit;
			}
		}
		else
		{
			if (MeshMaterial_Lit)
			{
				CurrentMaterial = MeshMaterial_Lit;
			}
		}

		int32 MaterialID = 0;
		ShapesMaterials.Reset();

		for (const FSVGShapeParameters& ShapeParameters : ShapeParametersList)
		{
			UMaterialInstanceDynamic* NewMaterial = CreateAndSetMaterialInstanceDynamicFromMaterial(MaterialID, CurrentMaterial);

			FLinearColor ColorToApply = FLinearColor::Black;

			switch (Coloring)
			{
				case EJoinedSVGMeshColoring::SeparateColors:
					ColorToApply = ShapeParameters.Color;
					break;

				case EJoinedSVGMeshColoring::SingleColor:
					ColorToApply = MainColor;
					break;
			}

			NewMaterial->SetVectorParameterValue("Color", ColorToApply);
			ShapesMaterials.Add(NewMaterial);
			MaterialID++;
		}
	}
	else
	{
		LoadMaterialSetParameters();
	}

	TArray<UMaterialInterface*> MaterialSet;
	MaterialSet.Append(ShapesMaterials);

	ConfigureMaterialSet(MaterialSet);
}
