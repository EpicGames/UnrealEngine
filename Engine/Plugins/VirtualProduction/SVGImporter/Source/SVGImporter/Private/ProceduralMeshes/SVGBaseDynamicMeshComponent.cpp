// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshes/SVGBaseDynamicMeshComponent.h"
#include "SVGEngineSubsystem.h"

USVGBaseDynamicMeshComponent::USVGBaseDynamicMeshComponent()
{
	bMeshHasBeenUpdated = false;
}

void USVGBaseDynamicMeshComponent::OnRegister()
{
	Super::OnRegister();

	if (bMeshHasBeenUpdated)
	{
		if (AActor* Owner = GetOwner())
		{
			USVGEngineSubsystem::OnSVGShapesUpdated().Execute(Owner);
		}

		bMeshHasBeenUpdated = false;
	}
}

void USVGBaseDynamicMeshComponent::MarkSVGMeshUpdated()
{
	bMeshHasBeenUpdated = true;
}
