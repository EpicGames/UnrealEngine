// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "GeometryScript/MeshSimplifyFunctions.h"
#include "GeometryScript/MeshTransformFunctions.h"
#include "GeometryScript/MeshUVFunctions.h"
#include "Interfaces/IPluginManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/Paths.h"
#include "SVGActor.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Misc/TransactionObjectEvent.h"
#include "SVGImporterUtils.h"
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "SVGDynamicMeshComponent"

USVGDynamicMeshComponent::USVGDynamicMeshComponent()
{
	LoadResources();

	SetCastShadow(false);

	Extrude = 0.0f;
	DefaultExtrude = Extrude;

	MinExtrudeValue = 0.0f;
	LastValueSetMinExtrude = MinExtrudeValue;
	LastValueSetExtrude = Extrude;

	ExtrudeDirection = FVector::ForwardVector;
	CurrentEditMode = ESVGEditMode::ValueSet;

	Bevel = 0.0f;
	Scale = 1.0f;

#if WITH_EDITORONLY_DATA
	bIsBevelBeingEdited = false;
#endif

	SVGStoredMesh = CreateDefaultSubobject<UDynamicMesh>("SVGStoredMesh");
}

void USVGDynamicMeshComponent::FlattenShape()
{
	EditMesh([this](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.VertexCount() == 0)
		{
			return;
		}

		// SVG shapes are always slightly extruded, either towards -X, or symmetrically towards -X and +X
		// In order to flatten SVG shapes:
		//   1- Translate them by the right amount along X, so that one of the main faces rests on the YZ plane.
		//      This leads to vertices with X values close or equal to 0.0f
		//   2- Remove all the vertices not on that plane (that is, with X other than ~0.0f)
		//   3- Force all other vertices which already are close to the YZ plane to be exactly on it (set X to 0.0f)

		if (ExtrudeType == ESVGExtrudeType::FrontBackMirror)
		{
			MeshTransforms::Translate(EditMesh, FVector3d(-GetExtrudeDepth() * 0.5f, 0.0f, 0.0f));
		}

		// Select only those points which are close enough to YZ
		for (const int32 VId : EditMesh.VertexIndicesItr())
		{
			FVector3d Vertex = EditMesh.GetVertex(VId);

			if (FMath::IsNearlyZero(Vertex.X, UE_KINDA_SMALL_NUMBER))
			{
				Vertex.X = 0.0f;
				EditMesh.SetVertex(VId, Vertex);
			}
			else
			{
				EditMesh.RemoveVertex(VId);
			}
		}
	});

	StoreCurrentMesh();

	ExtrudeType = ESVGExtrudeType::None;
}

void USVGDynamicMeshComponent::ApplyScale()
{
	EditMesh([this](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.VertexCount() == 0)
		{
			return;
		}

		const FVector3d ScaleVector(Scale);
		const FVector3d Origin = -InternalCenter;

		MeshTransforms::Scale(EditMesh, ScaleVector, Origin);

		FVector3d Center = EditMesh.GetBounds().Center();
		Center.X = 0.0f;

		MeshTransforms::Translate(EditMesh, -Center);
	});

	if (Cast<ASVGActor>(GetOwner()))
	{
		SetRelativeLocation(FVector(0.0f, InternalCenter.Y, InternalCenter.Z) * Scale);
	}
}

void USVGDynamicMeshComponent::ScaleShape(float InScale)
{
	if (!FMath::IsNearlyEqual(Scale,InScale))
	{
		Scale = InScale;

		LoadStoredMesh();
		ApplyScale();
	}
}

void USVGDynamicMeshComponent::Simplify()
{
	constexpr FGeometryScriptPlanarSimplifyOptions SimplifyOptions;
	UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPlanar(GetDynamicMesh(), SimplifyOptions);
}

void USVGDynamicMeshComponent::CenterMesh()
{
	UDynamicMesh* DynamicMesh = GetDynamicMesh();
	if (!DynamicMesh)
	{
		return;
	}

	// store location delta, in case the user moved the component from its original position, so we can re-apply the change
	const FVector LocationDelta = GetRelativeLocation() - InternalCenter;

	const UE::Geometry::FDynamicMesh3* DynamicMeshPtr = DynamicMesh->GetMeshPtr();
	if (!DynamicMeshPtr)
	{
		return;
	}

	UE::Math::TVector<double> Center = DynamicMeshPtr->GetBounds().Center();

	if (ExtrudeType == ESVGExtrudeType::FrontFaceOnly)
	{
		Center -= FVector(0.0f, 0.0f, Center.Z);
	}

	// center the mesh + rotate so that the pivot is in the middle
	const FTransform TranslationTr(-Center);
	const FTransform RotMeshTransform(FRotator(90.0f, 0.0f, 0.0f));
	const FTransform CenterMeshTransform = TranslationTr * RotMeshTransform;
	UGeometryScriptLibrary_MeshTransformFunctions::TransformMesh(DynamicMesh, CenterMeshTransform);

	// like this though, all meshes will be at the center of the actor - we need to offset them again on YZ axis
	InternalCenter = FVector(0.0f, Center.Y, Center.X);

	// move component to its location, and also add a location delta, in case the user moved the fill component already
	SetRelativeLocation(InternalCenter + LocationDelta);
}

void USVGDynamicMeshComponent::InitializeFromSVGDynamicMesh(const USVGDynamicMeshComponent* InOtherSVGDynamicMeshComponent)
{
	Color = InOtherSVGDynamicMeshComponent->Color;
	ExtrudeType = InOtherSVGDynamicMeshComponent->ExtrudeType;
	Bevel = InOtherSVGDynamicMeshComponent->Bevel;
	SVGColor = InOtherSVGDynamicMeshComponent->SVGColor;
	DefaultExtrude = InOtherSVGDynamicMeshComponent->DefaultExtrude;
	ExtrudeDirection = InOtherSVGDynamicMeshComponent->ExtrudeDirection;
	bSVGIsUnlit = InOtherSVGDynamicMeshComponent->bSVGIsUnlit;
	MaterialType = InOtherSVGDynamicMeshComponent->MaterialType;

	// We will apply the scale where needed, and set it to 1.0f
	const float SourceScale = InOtherSVGDynamicMeshComponent->Scale;
	Extrude = InOtherSVGDynamicMeshComponent->Extrude * SourceScale;
	MinExtrudeValue = InOtherSVGDynamicMeshComponent->MinExtrudeValue * SourceScale;
	Scale = 1.0f;

	CreateSVGMaterialInstance();

	InOtherSVGDynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& SourceEditMesh)
	{
		EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh = SourceEditMesh;
		});
	});

	StoreCurrentMesh();
}

void USVGDynamicMeshComponent::LoadResources()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("SVGImporter"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		const FString ContentRoot = Plugin->GetMountedAssetPath();
		const FString SVGMaterialsPath = FPaths::Combine(TEXT("Material'"), ContentRoot, TEXT("Materials"));

		const FString SVGMainMaterialPath       = FPaths::Combine(SVGMaterialsPath, TEXT("SVGMainMaterial.SVGMainMaterial'"));
		const FString SVGMainMaterialPath_Unlit = FPaths::Combine(SVGMaterialsPath, TEXT("SVGMainMaterial_Unlit.SVGMainMaterial_Unlit'"));

		static ConstructorHelpers::FObjectFinder<UMaterial> BasicTextMaterialFinder(*SVGMainMaterialPath);

		if (BasicTextMaterialFinder.Succeeded())
		{
			MeshMaterial = BasicTextMaterialFinder.Object;
		}

		static ConstructorHelpers::FObjectFinder<UMaterial> BasicTextMaterialUnlitFinder(*SVGMainMaterialPath_Unlit);

		if (BasicTextMaterialUnlitFinder.Succeeded())
		{
			MeshMaterial_Unlit = BasicTextMaterialUnlitFinder.Object;
		}
	}
}

void USVGDynamicMeshComponent::ApplyExtrudePreview()
{
	float FinalExtrude = GetExtrudeDepth();

	if (FMath::IsNearlyZero(FinalExtrude))
	{
		FinalExtrude = 0.1f;
	}

	FinalExtrude = FinalExtrude / (LastValueSetMinExtrude + LastValueSetExtrude);

	// for interactive mode, we use scale instead of actually modifying the geometry, to give responsive feedback to the user
	const FVector ScaleVec = FVector::OneVector - ExtrudeDirection + (ExtrudeDirection * FinalExtrude); 
	SetRelativeScale3D(ScaleVec);
}

void USVGDynamicMeshComponent::SetExtrudeInternal(float InExtrude)
{
	Extrude = InExtrude;

	if (CurrentEditMode == ESVGEditMode::Interactive)
	{
		ApplyExtrudePreview();
	}
	else if (CurrentEditMode == ESVGEditMode::ValueSet)
	{
		// reset scale - we are actually extruding the mesh
		SetRelativeScale3D(FVector::OneVector);

		LastValueSetExtrude = InExtrude;
		LastValueSetMinExtrude = MinExtrudeValue;

		// in this case we actually modify the geometry!
		const bool bSuccess = OnApplyMeshExtrudeDelegate.ExecuteIfBound();

		SetMeshEditMode(ESVGEditMode::Interactive);

		if (!bSuccess)
		{
			ApplyExtrudePreview();
		}
	}
}

void USVGDynamicMeshComponent::SetColor(FColor InColor)
{
	if (!MeshMaterialInstance)
	{
		CreateSVGMaterialInstance();
	}

	if (MeshMaterialInstance)
	{
		MeshMaterialInstance->SetVectorParameterValue("Color", InColor);
		AssignedMaterial = nullptr;

		SetMaterial(0, MeshMaterialInstance);
	}

	Color = InColor;
}

void USVGDynamicMeshComponent::CreateSVGMaterialInstance()
{
	if (bSVGIsUnlit)
	{
		if (MeshMaterial_Unlit && !MeshMaterialInstance && MaterialType == ESVGMaterialType::Default)
		{
			MeshMaterialInstance = CreateAndSetMaterialInstanceDynamicFromMaterial(0, MeshMaterial_Unlit);
			MeshMaterialInstance->SetVectorParameterValue("Color", SVGColor);
		}
	}
	else
	{
		if (MeshMaterial && !MeshMaterialInstance && MaterialType == ESVGMaterialType::Default)
		{
			MeshMaterialInstance = CreateAndSetMaterialInstanceDynamicFromMaterial(0, MeshMaterial);
			MeshMaterialInstance->SetVectorParameterValue("Color", SVGColor);
		}
	}
}

void USVGDynamicMeshComponent::SetMinExtrudeValue(float InMinExtrude)
{
	MinExtrudeValue = InMinExtrude;
	RefreshExtrude();
}

void USVGDynamicMeshComponent::SetBevel(float InBevel)
{
	Bevel = InBevel;
	RefreshBevel();
}

void USVGDynamicMeshComponent::StoreCurrentMesh() const
{
	ProcessMesh([this](const FDynamicMesh3& SourceMesh)
	{
		SVGStoredMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh = SourceMesh;
		});
	});
}

void USVGDynamicMeshComponent::RefreshBevel()
{
	if (CurrentEditMode == ESVGEditMode::Interactive)
	{
#if WITH_EDITORONLY_DATA
		bIsBevelBeingEdited = true;
#endif
	}
	else if (CurrentEditMode == ESVGEditMode::ValueSet)
	{
#if WITH_EDITORONLY_DATA
		// might need to undo some interactive display stuff
		bIsBevelBeingEdited = false;
#endif

		// in this case we actually modify the geometry!
		OnApplyMeshBevelDelegate.ExecuteIfBound();
	}
}

void USVGDynamicMeshComponent::LoadStoredMesh()
{
	if (!SVGStoredMesh->IsEmpty())
	{
		EditMesh([this](FDynamicMesh3& EditMesh)
		{
			EditMesh = SVGStoredMesh->GetMeshRef();
		});
	}
	else
	{
		RegenerateMesh();
		StoreCurrentMesh();
	}

	MarkSVGMeshUpdated();
}

void USVGDynamicMeshComponent::PostLoad()
{
	Super::PostLoad();

	SetMeshEditMode(ESVGEditMode::ValueSet);
	LoadStoredMesh();
	ApplyScale();
	RefreshMaterial();
	RegisterDelegates();
}

void USVGDynamicMeshComponent::PostInitProperties()
{
	Super::PostInitProperties();
	RegisterDelegates();
}

void USVGDynamicMeshComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	SetMeshEditMode(ESVGEditMode::ValueSet);
	LoadStoredMesh();
	ApplyScale();

	RegisterDelegates();
}

#if WITH_EDITOR
void USVGDynamicMeshComponent::ResetToSVGValues()
{
	FScopedTransaction Transaction(LOCTEXT("ChangeSVGMeshColor", "Change SVG Mesh Color"));
	this->Modify();

	SetColor(SVGColor);

	// Will not reset base extrude!
	SetExtrudeInternal(DefaultExtrude);
}

void USVGDynamicMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName ColorName = GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, Color);
	const FName ExtrudeName = GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, Extrude);
	const FName OverrideMaterialsName = GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, OverrideMaterials);
	const FName BaseMaterialsName = GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, BaseMaterials);
	const FName UnlitName = GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, bSVGIsUnlit);

	const FName ChangedPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	const ESVGEditMode EditMode = PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet ? ESVGEditMode::ValueSet : ESVGEditMode::Interactive;
	SetMeshEditMode(EditMode);

	if (ChangedPropertyName == ColorName)
	{
		RefreshColor();
	}
	else if (ChangedPropertyName == ExtrudeName)
	{
		RefreshExtrude();
	}
	else if (ChangedPropertyName == OverrideMaterialsName || ChangedPropertyName == BaseMaterialsName)
	{
		RefreshCustomMaterial();
	}
	else if (ChangedPropertyName == UnlitName)
	{
		MeshMaterialInstance = nullptr;
		RefreshMaterial();
	}
}

void USVGDynamicMeshComponent::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.HasPropertyChanges())
	{
		const TArray<FName>& ChangedProperties = InTransactionEvent.GetChangedProperties();

		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, MeshMaterialInstance)) ||
				 ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, AssignedMaterial)) ||
				 ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, Color)) ||
				 ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, SVGColor)) ||
				 ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGDynamicMeshComponent, bSVGIsUnlit)))
		{
			RefreshMaterial();
		}
	}
}

void USVGDynamicMeshComponent::PostEditImport()
{
	Super::PostEditImport();

	SetMeshEditMode(ESVGEditMode::ValueSet);
	LoadStoredMesh();
	ApplyScale();

	RegisterDelegates();
}

UStaticMesh* USVGDynamicMeshComponent::BakeStaticMesh()
{
	TMap<FString, UMaterialInstance*> GeneratedMaterialInstances;
	const FString SavePath = TEXT("/Game/SVG/Baked/");
	return FSVGImporterUtils::BakeSVGDynamicMeshToStaticMesh(this, SavePath, GeneratedMaterialInstances);
}
#endif

void USVGDynamicMeshComponent::RefreshCustomMaterial()
{
	if (!BaseMaterials.IsEmpty())
	{
		MaterialType = ESVGMaterialType::Custom;
		AssignedMaterial = BaseMaterials[0];
		MeshMaterialInstance = nullptr;
		SetCustomMaterial();
	}
}

void USVGDynamicMeshComponent::RefreshColor()
{
	MaterialType = ESVGMaterialType::Default;
	SetColor(Color);
}

void USVGDynamicMeshComponent::RefreshExtrude()
{
	if (FMath::IsNearlyZero(Extrude))
	{
		Extrude = 0.01f;
	}

	SetExtrudeInternal(Extrude);
}

void USVGDynamicMeshComponent::SetCustomMaterial()
{
	if (AssignedMaterial)
	{
		SetMaterial(0, AssignedMaterial);
	}
}

void USVGDynamicMeshComponent::RefreshMaterial()
{
	switch (MaterialType)
	{
		case ESVGMaterialType::Default:
			SetColor(Color);
			break;

		case ESVGMaterialType::Custom:
			SetCustomMaterial();
			break;
	}

	MarkRenderStateDirty();
}

void USVGDynamicMeshComponent::SetIsUnlit(bool bInIsUnlit)
{
	bSVGIsUnlit = bInIsUnlit;
	MeshMaterialInstance = nullptr;
	RefreshMaterial();
}

void USVGDynamicMeshComponent::ApplySimpleUVPlanarProjection()
{
	static const FTransform UVProjPlaneTransform(FRotator(0.0f, 90.0f, 0.0f));
	const FGeometryScriptMeshSelection Selection;
	UGeometryScriptLibrary_MeshUVFunctions::SetMeshUVsFromPlanarProjection(GetDynamicMesh(), 0, UVProjPlaneTransform, Selection);
}

void USVGDynamicMeshComponent::SetExtrudeType(const ESVGExtrudeType InExtrudeType)
{
	ExtrudeType = InExtrudeType;
}

void USVGDynamicMeshComponent::SetMeshEditMode(const ESVGEditMode InEditMode)
{
	CurrentEditMode = InEditMode;
}

#undef LOCTEXT_NAMESPACE
