// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshes/SVGFillComponent.h"
#include "Curve/PolygonOffsetUtils.h"
#include "GeometryScript/MeshBooleanFunctions.h"
#include "GeometryScript/MeshModelingFunctions.h"
#include "GeometryScript/MeshNormalsFunctions.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"
#include "GeometryScript/MeshTransformFunctions.h"
#include "ProceduralMeshes/SVGStrokeComponent.h"

#if WITH_EDITOR
#include "Misc/TransactionObjectEvent.h"
#endif

void FSVGFillMeshData::Init(const TArray<FSVGPathPolygon>& InShapes)
{
	ShapesToDraw.Reset(InShapes.Num());
	for (const FSVGPathPolygon& Shape : InShapes)
	{
		FSVGFillShape FillShape(Shape.Get2DVertices(), Shape.GetShouldBeDrawn());
		ShapesToDraw.Emplace(FillShape);
	}
}

void USVGFillComponent::GenerateFillMesh(const FSVGFillParameters& InFillParameters)
{
	FillMeshData.Init(InFillParameters.Shapes);

	SVGColor = InFillParameters.Color;
	Color = SVGColor;
	ExtrudeDirection = FVector::ForwardVector;
	DefaultExtrude = InFillParameters.Extrude;
	Extrude = DefaultExtrude;
	Bevel = InFillParameters.BevelDistance;
	bSimplify = InFillParameters.bSimplify;
	bBypassClipper = !InFillParameters.bSmoothShapes;
	SmoothingOffset = InFillParameters.SmoothingOffset;
	bSVGIsUnlit = InFillParameters.bUnlit;

	SetCastShadow(InFillParameters.bCastShadow);

	GenerateFillMeshInternal();
	RegisterDelegates();
}

void USVGFillComponent::GenerateFillMeshInternal()
{
	InitBasePolygons();

	LoadCachedMeshToDynamicMesh(CachedBasePolygon);
	ApplyExtrude(GetExtrudeDepth());
	ApplySimplifyAndBevel();
	ApplyFinalMeshChanges();
	StoreCurrentMesh();
	ApplyScale();
	CreateSVGMaterialInstance();
}

void USVGFillComponent::InitBasePolygons()
{
	// cache the polygons used to represent the fills to be drawn, and the ones to be cut from those fills

	// Create a temporary dyn mesh to operate on, and store intermediate mesh data
	UDynamicMesh* DynamicMesh = NewObject<UDynamicMesh>();

	// main polygon mesh
	for (const FSVGFillShape& ShapeToDraw : FillMeshData.ShapesToDraw)
	{
		if (ShapeToDraw.ShapeVertices.Num() >= 3)
		{
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;

			// Add shape
			if (ShapeToDraw.bShouldBeDrawn)
			{
				if (bBypassClipper)
				{
					UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(DynamicMesh, PrimitiveOptions,  FTransform::Identity, ShapeToDraw.ShapeVertices);
				}
				else
				{
					UE::Geometry::FGeneralPolygon2d ShapePath{ShapeToDraw.ShapeVertices};
					TArray<UE::Geometry::FGeneralPolygon2d> OffsetResult;

					UE::Geometry::PolygonsOffset(SmoothingOffset, {ShapePath}, OffsetResult, true, 1.0, UE::Geometry::EPolygonOffsetJoinType::Round, UE::Geometry::EPolygonOffsetEndType::Polygon);

					if (!OffsetResult.IsEmpty())
					{
						UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(DynamicMesh, PrimitiveOptions, FTransform::Identity, OffsetResult[0].GetOuter().GetVertices());
					}
				}
			}
			// Cut hole
			else
			{
				UDynamicMesh* SubtractMesh = NewObject<UDynamicMesh>();
				UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(SubtractMesh, PrimitiveOptions, FTransform::Identity, ShapeToDraw.ShapeVertices);

				FTransform SubtractMeshTransform = FTransform::Identity;
				SubtractMeshTransform.SetTranslation(FVector::DownVector * 0.5f);
				UGeometryScriptLibrary_MeshTransformFunctions::TransformMesh(SubtractMesh, SubtractMeshTransform);

				FGeometryScriptMeshLinearExtrudeOptions ExtrudeOptions;
				ExtrudeOptions.Distance  = 1.0f;
				ExtrudeOptions.Direction = FVector::UpVector;

				UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshLinearExtrudeFaces(SubtractMesh
					, ExtrudeOptions
					, FGeometryScriptMeshSelection()
					, nullptr);

				FGeometryScriptMeshBooleanOptions SubtractOptions;
				SubtractOptions.bFillHoles = false;
				SubtractOptions.bSimplifyOutput = true;
				UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshBoolean(DynamicMesh, FTransform::Identity, SubtractMesh, FTransform::Identity, EGeometryScriptBooleanOperation::Subtract, SubtractOptions);
			}
		}
	}

	DynamicMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		CachedBasePolygon = Mesh;
	});
}

#if WITH_EDITOR
void USVGFillComponent::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.HasPropertyChanges() && TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		const TArray<FName>& ChangedProperties = TransactionEvent.GetChangedProperties();

		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGFillComponent, bBypassClipper)) ||
			ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGFillComponent, SmoothingOffset)))
		{
			GenerateFillMeshInternal();
		}
	}
}

void USVGFillComponent::PostEditImport()
{
	Super::PostEditImport();
	InitBasePolygons();
}
#endif

void USVGFillComponent::PostLoad()
{
	Super::PostLoad();
	InitBasePolygons();
}

void USVGFillComponent::RegisterDelegates()
{
	OnApplyMeshExtrudeDelegate.Unbind();
	OnApplyMeshExtrudeDelegate.BindUObject(this, &USVGFillComponent::ApplyFillExtrude);

	OnApplyMeshBevelDelegate.Unbind();
	OnApplyMeshBevelDelegate.BindUObject(this, &USVGFillComponent::ApplyFillBevel);
}

void USVGFillComponent::RegenerateMesh()
{
	Super::RegenerateMesh();
	GenerateFillMeshInternal();
}

void USVGFillComponent::RefreshNormals()
{
	UDynamicMesh* FillMesh = GetDynamicMesh();
	if (!FillMesh)
	{
		return;
	}

	constexpr FGeometryScriptCalculateNormalsOptions NormalsOptions;
	UGeometryScriptLibrary_MeshNormalsFunctions::RecomputeNormals(FillMesh, NormalsOptions);
}

void USVGFillComponent::LoadCachedMeshToDynamicMesh(const TOptional<UE::Geometry::FDynamicMesh3>& InSourceMesh)
{
	if (!InSourceMesh.IsSet())
	{
		return;
	}

	UDynamicMesh* FillMesh = GetDynamicMesh();
	if (!FillMesh)
	{
		return;
	}

	FillMesh->Reset();
	EditMesh([&InSourceMesh](FDynamicMesh3& EditMesh)
	{
		EditMesh = InSourceMesh.GetValue();
	});
}

void USVGFillComponent::ApplyExtrude(float InExtrudeValue)
{
	const UDynamicMesh* FillMesh = GetDynamicMesh();
	if (!FillMesh)
	{
		return;
	}

	if (!FMath::IsNearlyZero(InExtrudeValue))
	{
		FGeometryScriptMeshLinearExtrudeOptions ExtrudeOptions;
		ExtrudeOptions.Distance  = InExtrudeValue;
		ExtrudeOptions.Direction = FVector::UpVector;

		UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshLinearExtrudeFaces(GetDynamicMesh()
			, ExtrudeOptions
			, FGeometryScriptMeshSelection()
			, nullptr);
	}

	// Store extruded mesh, pre-bevel
	FillMesh->ProcessMesh([&](const FDynamicMesh3& SourceEditMesh)
	{
		CachedExtrudedMesh = SourceEditMesh;
	});
}

void USVGFillComponent::ApplySimplifyAndBevel()
{
	UDynamicMesh* FillMesh = GetDynamicMesh();
	if (!FillMesh)
	{
		return;
	}

	if (bSimplify)
	{
		Simplify();
	}

	if (Bevel > 0.0f)
	{
		FGeometryScriptMeshBevelOptions BevelOptions;
		BevelOptions.BevelDistance = Bevel;
		UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshPolygroupBevel(FillMesh, BevelOptions);
	}
}

void USVGFillComponent::ApplyFillExtrude()
{
	if (!CachedBasePolygon.IsSet())
	{
		InitBasePolygons();
	}

	LoadCachedMeshToDynamicMesh(CachedBasePolygon);
	ApplyExtrude(MinExtrudeValue + Extrude);
	ApplySimplifyAndBevel();
	ApplyFinalMeshChanges();
	StoreCurrentMesh();
	ApplyScale();
}

void USVGFillComponent::ApplyFinalMeshChanges()
{
	RefreshNormals();
	ApplySimpleUVPlanarProjection();
	CenterMesh();
}

void USVGFillComponent::ApplyFillBevel()
{
	if (!CachedExtrudedMesh.IsSet())
	{
		ApplyFillExtrude();
	}

	LoadCachedMeshToDynamicMesh(CachedExtrudedMesh);
	ApplySimplifyAndBevel();
	ApplyFinalMeshChanges();
	StoreCurrentMesh();
	ApplyScale();
}

void USVGFillComponent::SetSmoothFillShapes(bool bInSmoothFillShapes, float InSmoothingOffset)
{
	const bool bNewBypassClipper = !bInSmoothFillShapes;

	if ((bNewBypassClipper != bBypassClipper) || (InSmoothingOffset != SmoothingOffset))
	{
		SmoothingOffset = InSmoothingOffset;
		bBypassClipper = bNewBypassClipper;

		GenerateFillMeshInternal();
	}
}
