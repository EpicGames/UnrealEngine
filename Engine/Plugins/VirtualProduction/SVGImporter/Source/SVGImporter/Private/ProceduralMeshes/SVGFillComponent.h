// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "SVGTypes.h"
#include "SVGFillComponent.generated.h"

USTRUCT()
struct FSVGFillShape
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FVector2D> ShapeVertices;

	UPROPERTY()
	bool bShouldBeDrawn = true;

	FSVGFillShape(){}
	FSVGFillShape(const TArray<FVector2D>& InVertices, bool bInShouldBeDrawn)
	: ShapeVertices(InVertices)
	, bShouldBeDrawn(bInShouldBeDrawn)
	{}
};

USTRUCT()
struct FSVGFillMeshData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FSVGFillShape> ShapesToDraw;

	void Init(const TArray<FSVGPathPolygon>& InShapes);
};

struct FSVGFillParameters
{
	TArray<FSVGPathPolygon> Shapes;

	FColor Color;

	float Extrude = 0.0f;

	bool bSimplify = false;

	float BevelDistance = 0.0f;

	bool bSmoothShapes = false;

	float SmoothingOffset = 150.0f;

	bool bUnlit = false;

	bool bCastShadow = false;

	FSVGFillParameters(const TArray<FSVGPathPolygon>& InShapes)
		:Shapes(InShapes)
	{
	}
};

UCLASS(ClassGroup=(SVGImporter), Meta = (BlueprintSpawnableComponent))
class USVGFillComponent : public USVGDynamicMeshComponent
{
	GENERATED_BODY()

public:
	void GenerateFillMesh(const FSVGFillParameters& InFillParameters);
	void SetSmoothFillShapes(bool bInSmoothFillShapes, float InSmoothingOffset);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	virtual void PostEditImport() override;
#endif
	virtual void PostLoad() override;
	//~ End UObject

	//~ Begin USVGDynamicMesh
	virtual void RegisterDelegates() override;
	virtual void RegenerateMesh() override;
	virtual FName GetShapeType() const override { return TEXT("Fill"); }
	//~ End USVGDynamicMesh

	void RefreshNormals();
	void InitBasePolygons();
	void GenerateFillMeshInternal();

	void LoadCachedMeshToDynamicMesh(const TOptional<UE::Geometry::FDynamicMesh3>& InSourceMesh);
	void ApplyExtrude(float InExtrudeValue);
	void ApplySimplifyAndBevel();

	/** Applies all changes after extrusion. Includes beveling. */
	void ApplyFinalMeshChanges();

	/** Applies all changes from extrusion on. */
	void ApplyFillExtrude();

	/** Applies changes from extrude on. */
	void ApplyFillBevel();

	TOptional<UE::Geometry::FDynamicMesh3> CachedExtrudedMesh;
	TOptional<UE::Geometry::FDynamicMesh3> CachedBasePolygon;

	UPROPERTY()
	bool bSimplify;

	UPROPERTY()
	bool bBypassClipper;

	UPROPERTY()
	float SmoothingOffset;

	UPROPERTY()
	FSVGFillMeshData FillMeshData;
};
