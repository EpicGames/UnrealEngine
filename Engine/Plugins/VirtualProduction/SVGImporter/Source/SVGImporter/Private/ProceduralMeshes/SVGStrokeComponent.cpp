// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralMeshes/SVGStrokeComponent.h"
#include "Curve/PolygonOffsetUtils.h"
#include "GeometryScript/MeshBooleanFunctions.h"
#include "GeometryScript/MeshModelingFunctions.h"
#include "GeometryScript/MeshNormalsFunctions.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"
#include "GeometryScript/MeshSimplifyFunctions.h"
#include "GeometryScript/MeshTransformFunctions.h"

#if WITH_EDITOR
#include "Misc/TransactionObjectEvent.h"
#endif

void USVGStrokeComponent::GenerateStrokeMesh(const FSVGStrokeParameters& InStrokeParameters)
{
	SVGColor = InStrokeParameters.InColor;
	Color = SVGColor;
	bStrokeIsClosed = InStrokeParameters.bIsClosed;
	bStrokeIsClockwise = InStrokeParameters.bIsClockwise;
	StrokePoints = InStrokeParameters.StrokePoints;
	StrokeThickness = InStrokeParameters.Thickness;
	DefaultExtrude = InStrokeParameters.Extrude;
	Extrude = DefaultExtrude;
	JoinStyle = InStrokeParameters.JoinStyle;
	bSVGIsUnlit = InStrokeParameters.bUnlit;

	SetCastShadow(InStrokeParameters.bCastShadow);

	ExtrudeDirection = FVector::ForwardVector;

	// polygon should be counterclockwise
	if (bStrokeIsClockwise)
	{
		Algo::Reverse(StrokePoints);
		bStrokeIsClockwise = false;
	}

	if (StrokePoints.Num() > 3)
	{
		if (StrokePoints[0] == StrokePoints.Last())
		{
			bStrokeIsClosed = true;
		}
	}

	GenerateStrokeMeshInternal();
}

void USVGStrokeComponent::SetJointStyle(EPolygonOffsetJoinType InJoinStyle)
{
	JoinStyle = InJoinStyle;
}

void USVGStrokeComponent::SetStrokeWidth(float InStrokesWidth)
{
	if (InStrokesWidth != StrokeThickness)
	{
		StrokeThickness = InStrokesWidth;

		GenerateStrokeMeshInternal();
	}
}

#if WITH_EDITOR
void USVGStrokeComponent::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.HasPropertyChanges() && TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		const TArray<FName>& ChangedProperties = TransactionEvent.GetChangedProperties();

		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGStrokeComponent, StrokeThickness)) ||
			ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(USVGStrokeComponent, JoinStyle)))
		{
			GenerateStrokeMeshInternal();
		}
	}
}
#endif

void USVGStrokeComponent::GenerateStrokeMeshInternal()
{
	UDynamicMesh* StrokeMesh = GetDynamicMesh();
	if (!StrokeMesh)
	{
		return;
	}

	StrokeMesh->Reset();
	GenerateMainStrokeGeometry(StrokeThickness, MinExtrudeValue + Extrude);

	if (JoinStyle == EPolygonOffsetJoinType::Square)
	{
		CleanupStrokeMesh();
	}

	CreateSVGMaterialInstance();
	ApplySimpleUVPlanarProjection();
	CenterMesh();

	StoreCurrentMesh();
	ApplyScale();

	RegisterDelegates();
}

void USVGStrokeComponent::GenerateMainStrokeGeometry(float InThickness, float InExtrude)
{
	UDynamicMesh* StrokeMesh = GetDynamicMesh();
	if (!StrokeMesh)
	{
		return;
	}

	TArray<FVector2d> StrokePoints2d;
	StrokePoints2d.Reserve(StrokePoints.Num());
	Algo::Transform(StrokePoints, StrokePoints2d, [](const FVector& InPoint)
	{
		return FVector2d(InPoint.Z, InPoint.Y);
	});

	TArray<UE::Geometry::FGeneralPolygon2d> StrokePathToExtrude_Offset;
	TArray<UE::Geometry::FGeneralPolygon2d> StrokePathToExtrude_Inset;

	UE::Geometry::FPolygon2d StrokePathToExtrude(StrokePoints2d);

	float OffsetDelta = InThickness;

	if (bStrokeIsClosed)
	{
		OffsetDelta *= 0.5f;
	}

	// If the line is closed, the end cap type is polygon
	UE::Geometry::EPolygonOffsetEndType EndCapType = UE::Geometry::EPolygonOffsetEndType::Polygon;

	// otherwise set depending on join style
	if (!bStrokeIsClosed)
	{
		switch (JoinStyle)
		{
		case EPolygonOffsetJoinType::Square:
			EndCapType = UE::Geometry::EPolygonOffsetEndType::Square;
			break;

		case EPolygonOffsetJoinType::Round:
			EndCapType = UE::Geometry::EPolygonOffsetEndType::Round;
			break;

		case EPolygonOffsetJoinType::Miter:
			EndCapType = UE::Geometry::EPolygonOffsetEndType::Butt;
			break;

		default: ;
		}
	}

	// we use offset AND inset in order to center the stroke around the actual path
	UE::Geometry::PolygonsOffset(OffsetDelta, { StrokePathToExtrude }, StrokePathToExtrude_Offset, true, 1.0, static_cast<UE::Geometry::EPolygonOffsetJoinType>(JoinStyle), EndCapType);
	UE::Geometry::PolygonsOffset(-OffsetDelta, { StrokePathToExtrude }, StrokePathToExtrude_Inset, true, 1.0, static_cast<UE::Geometry::EPolygonOffsetJoinType>(JoinStyle), EndCapType);

	if (!StrokePathToExtrude_Offset.IsEmpty())
	{
		const FGeometryScriptPrimitiveOptions PrimitiveOptions;
		UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(StrokeMesh, PrimitiveOptions, FTransform::Identity, StrokePathToExtrude_Offset[0].GetOuter().GetVertices());
	}

	if (!FMath::IsNearlyZero(InExtrude))
	{
		FGeometryScriptMeshLinearExtrudeOptions ExtrudeOptions;
		ExtrudeOptions.Distance  = InExtrude;
		ExtrudeOptions.Direction = FVector::UpVector;

		UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshLinearExtrudeFaces(StrokeMesh
			, ExtrudeOptions
			, FGeometryScriptMeshSelection()
			, nullptr);
	}

	// if shape is closed, we need to subtract the inner shape resulting from the stroke extrude
	if (bStrokeIsClosed && !StrokePathToExtrude_Inset.IsEmpty())
	{
		UDynamicMesh* MeshToSubtract = NewObject<UDynamicMesh>();

		const FGeometryScriptPrimitiveOptions MeshToSubtractPrimitiveOptions;
		UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(
			MeshToSubtract
			, MeshToSubtractPrimitiveOptions
			, FTransform::Identity
			, StrokePathToExtrude_Inset[0].GetOuter().GetVertices());

		const float SubtractExtrude = FMath::IsNearlyZero(InExtrude) ? 1.0f : InExtrude * 2.0f;

		FTransform SubtractMeshTransform = FTransform::Identity;
		SubtractMeshTransform.SetTranslation(FVector::DownVector * SubtractExtrude * 0.5f);
		UGeometryScriptLibrary_MeshTransformFunctions::TransformMesh(MeshToSubtract, SubtractMeshTransform);

		FGeometryScriptMeshLinearExtrudeOptions SubtractExtrudeOptions;
		SubtractExtrudeOptions.Distance  = SubtractExtrude; // twice the size, to be sure boolean operation works
		SubtractExtrudeOptions.Direction = FVector::UpVector;

		UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshLinearExtrudeFaces(MeshToSubtract
			, SubtractExtrudeOptions
			, FGeometryScriptMeshSelection()
			, nullptr);

		FGeometryScriptMeshBooleanOptions SubtractOptions;
		SubtractOptions.bFillHoles = false;
		SubtractOptions.bSimplifyOutput = true;

		UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshBoolean(StrokeMesh, FTransform::Identity, MeshToSubtract, FTransform::Identity, EGeometryScriptBooleanOperation::Subtract, SubtractOptions);
	}
}

void USVGStrokeComponent::CleanupStrokeMesh()
{
	UDynamicMesh* StrokeMesh = GetDynamicMesh();
	if (!StrokeMesh)
	{
		return;
	}

	constexpr FGeometryScriptPlanarSimplifyOptions SimplifyOptions;
	UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPlanar(StrokeMesh, SimplifyOptions);
	UGeometryScriptLibrary_MeshNormalsFunctions::SetPerFaceNormals(StrokeMesh); // avoiding rounded look of flat strokes
}

void USVGStrokeComponent::RegisterDelegates()
{
	OnApplyMeshExtrudeDelegate.Unbind();
	OnApplyMeshExtrudeDelegate.BindUObject(this, &USVGStrokeComponent::ApplyStrokeExtrude);
}

void USVGStrokeComponent::RegenerateMesh()
{
	Super::RegenerateMesh();
	GenerateStrokeMeshInternal();
}

void USVGStrokeComponent::ApplyStrokeExtrude()
{
	// extrude is based on stroke section, we have to re-generate
	GenerateStrokeMeshInternal();
}
