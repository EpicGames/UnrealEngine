// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "SVGStrokeComponent.generated.h"

struct FSVGStrokeParameters
{
	const TArray<FVector>& StrokePoints;

	float Thickness = 0;

	FColor InColor;

	bool bIsClosed = false;

	bool bIsClockwise = false;

	EPolygonOffsetJoinType JoinStyle{};

	float Extrude = 0;

	bool bUnlit = false;

	bool bCastShadow = false;

	FSVGStrokeParameters(const TArray<FVector>& InPoints)
		: StrokePoints(InPoints)
	{
	}
};

UCLASS(ClassGroup=(SVGImporter), Meta = (BlueprintSpawnableComponent))
class USVGStrokeComponent : public USVGDynamicMeshComponent
{
	GENERATED_BODY()

public:
	void GenerateStrokeMesh(const FSVGStrokeParameters& InStrokeParameters);

	void SetJointStyle(EPolygonOffsetJoinType InJoinStyle);
	void SetStrokeWidth(float InStrokesWidth);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject

	//~ Begin USVGDynamicMesh
	virtual void RegisterDelegates() override;
	virtual void RegenerateMesh() override;
	virtual FName GetShapeType() const override { return TEXT("Stroke"); }
	//~ End USVGDynamicMesh

	void GenerateStrokeMeshInternal();
	void GenerateMainStrokeGeometry(float InThickness, float InExtrude);
	void ApplyStrokeExtrude();

	void CleanupStrokeMesh();

	UPROPERTY()
	TArray<FVector> StrokePoints;

	UPROPERTY()
	float StrokeThickness;

	UPROPERTY()
	bool bStrokeIsClosed;

	UPROPERTY()
	bool bStrokeIsClockwise;

	UPROPERTY()
	EPolygonOffsetJoinType JoinStyle;
};
