// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGActor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Interfaces/IPluginManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/Paths.h"
#include "ProceduralMeshes/SVGFillComponent.h"
#include "ProceduralMeshes/SVGStrokeComponent.h"
#include "SVGData.h"
#include "SVGEngineSubsystem.h"
#include "SVGImporter.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "LevelEditor.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "SVGActorEditorComponent.h"
#include "SVGImporterUtils.h"
#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

struct FSVGActorVersion
{
	enum Type : int32
	{
		PreVersioning = 0,

		/** ShapeComponents array now tracks both Fill and Stroke shapes instead of separated arrays */
		ShapeComponents,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x73ED0078, 0x781147BC, 0x8082B640, 0x9AA13DF0);
};

FCustomVersionRegistration GRegisterSVGActorVersion(FSVGActorVersion::GUID, static_cast<int32>(FSVGActorVersion::LatestVersion), TEXT("SVGActorVersion"));

namespace UE::SVGImporter::Private
{
	constexpr static bool bSimplifyFills = true;

	constexpr static float ExtrudeDepthIncrement  = 0.002f;
	constexpr static float MaxExtrudeDepthTotal   = 0.1f;
	constexpr static float GeneratedMeshesPerLoop = 1.0f;
	constexpr static float BevelUpdatesPerFrame   = 5.0f;
	constexpr static float DefaultShapesExtrude   = 0.5f;
	constexpr static float DefaultSmoothOffset    = 0.15f;
	constexpr static float DefaultBevelDistance   = 0.0f;
	constexpr static float DefaultShapesOffset    = 0.0f;
	constexpr static float OffsetScaleMultiplier  = 0.025f;

	constexpr static float DefaultNotificationsExpireTime  = 2.5f;
	constexpr static float DefaultNotificationsFadeOutTime = 0.75f;

	/** Used to ensure strokes are always slightly in front of fills within the same shape */
	constexpr static float StrokeAddedDepthMultiplier = 1.01f;

	static bool bAllowInitialization = true; 
}

#define LOCTEXT_NAMESPACE "SVGActor"

FSVGActorInitGuard::FSVGActorInitGuard()
{
	using namespace UE::SVGImporter;

	bPreviousValue = Private::bAllowInitialization;
	Private::bAllowInitialization = false;
}

FSVGActorInitGuard::~FSVGActorInitGuard()
{
	UE::SVGImporter::Private::bAllowInitialization = bPreviousValue;
}

// Sets default values
ASVGActor::ASVGActor()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;
	RootComponent = CreateDefaultSubobject<USceneComponent>("RootComponent");

	StrokeShapesRoot = CreateDefaultSubobject<USceneComponent>("StrokesRoot");
	StrokeShapesRoot->SetupAttachment(RootComponent);

	FillShapesRoot = CreateDefaultSubobject<USceneComponent>("FillsRoot");
	FillShapesRoot->SetupAttachment(RootComponent);

#if WITH_EDITOR
	SVGEditorComponent = CreateDefaultSubobject<USVGActorEditorComponent>("SVGEditorComponent");
	SVGEditorComponent->SetupAttachment(RootComponent);
#endif

	bMeshesShouldBeGenerated = true;
	CurrExtrudeForDepth = 0.0f;

	ResetShapesExtrudes();
	RenderMode = ESVGRenderMode::DynamicMesh3D;
	ExtrudeType = ESVGExtrudeType::FrontFaceOnly;

	bSVGHasFills = false;
	bSVGHasStrokes = false;
	bIgnoreStrokes = false;

	bSmoothFillShapes = false;

	StrokesWidth = 0.25f;
	StrokeJoinStyle = EPolygonOffsetJoinType::Round;

	UnnamedFillsNum = 0;
	UnnamedStrokesNum = 0;

	SmoothingOffset = UE::SVGImporter::Private::DefaultSmoothOffset;
	FillsExtrude = UE::SVGImporter::Private::DefaultShapesExtrude;
	StrokesExtrude = UE::SVGImporter::Private::DefaultShapesExtrude;
	BevelDistance = UE::SVGImporter::Private::DefaultBevelDistance;
	ShapesOffset =  UE::SVGImporter::Private::DefaultShapesOffset;
	ShapesOffsetScale = 1.0f;
	Scale = 1.0f;

	bClearInstanceComponents = false;

	bIsGeneratingMeshes = false;
	bIsUpdatingFills = false;
	bStrokesExtrudeFinished = false;
	bFillsExtrudeFinished = false;

	bSVGIsUnlit = true;
	bSVGCastsShadow = false;

	bInitialized = false;

	LoadResources();
}

void ASVGActor::Initialize()
{
	using namespace UE::SVGImporter;

	if (Private::bAllowInitialization && !bInitialized)
	{
		ApplyScaleAndCenter();
		ApplyOffset();

		TryLoadDefaultSVGData();

		if (SVGData)
		{
			if (bMeshesShouldBeGenerated)
			{
				Generate();
			}
		}

		bInitialized = true;
	}
}

void ASVGActor::TryLoadDefaultSVGData()
{
	if (!SVGData)
	{
		SVGData = FSVGImporterModule::Get().CreateDefaultSVGData();

		if (SVGData)
		{
			SVGData->Rename(nullptr, this);
			SVGData->ClearFlags(RF_Transient);
		}
	}
}

void ASVGActor::LoadResources()
{
	static ConstructorHelpers::FObjectFinder<UStaticMesh> PlaneMeshFinder(TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Plane.Plane'"));	
	if (PlaneMeshFinder.Succeeded())
	{
		PlaneStaticMesh = PlaneMeshFinder.Object;
	}

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("SVGImporter"));	
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		const FString ContentRoot = Plugin->GetMountedAssetPath();
		const FString SVGMaterialsPath = FPaths::Combine(TEXT("Material'"), ContentRoot, TEXT("Materials"));

		const FString SVGTextureMaterialPath       = FPaths::Combine(SVGMaterialsPath, TEXT("SVGTextureMaterial.SVGTextureMaterial'"));
		const FString SVGTextureMaterialPath_Unlit = FPaths::Combine(SVGMaterialsPath, TEXT("SVGTextureMaterial_Unlit.SVGTextureMaterial_Unlit'"));

		static ConstructorHelpers::FObjectFinder<UMaterial> PlaneMaterialFinder(*SVGTextureMaterialPath);
		if (PlaneMaterialFinder.Succeeded())
		{
			PlaneMaterial = PlaneMaterialFinder.Object;
		}

		static ConstructorHelpers::FObjectFinder<UMaterial> PlaneMaterialUnlitFinder(*SVGTextureMaterialPath_Unlit);
		if (PlaneMaterialUnlitFinder.Succeeded())
		{
			PlaneMaterial_Unlit = PlaneMaterialUnlitFinder.Object;
		}
	}
}

void ASVGActor::DestroySVGDynMeshComponents()
{
	FCoreDelegates::OnEndFrame.Remove(GenMeshDelegateHandle);

	ClearShapes();
}

void ASVGActor::RefreshPlaneMaterialInstance()
{
	if (bSVGIsUnlit)
	{
		PlaneMaterialInstance = UMaterialInstanceDynamic::Create(PlaneMaterial_Unlit, this);
	}
	else
	{
		PlaneMaterialInstance = UMaterialInstanceDynamic::Create(PlaneMaterial, this);
	}

	if (SVGData)
	{
		PlaneMaterialInstance->SetTextureParameterValue("Texture", SVGData->SVGTexture);
	}

	if (SVGPlane)
	{
		SVGPlane->SetMaterial(0, PlaneMaterialInstance);
	}
}

void ASVGActor::CreateSVGPlane()
{
	if (!ensureMsgf(SVGData, TEXT("SVG Data is Invalid")))
	{
#if WITH_EDITOR
		DisplayMissingSVGDataError(TEXT("Cannot generate SVG textured plane."));
#endif
		return;
	}

	DestroySVGDynMeshComponents();

	static const FName SVGPlaneName = FName(TEXT("SVGPlane_") + SVGName);
	const bool bPlaneActorNeedsToBeCreated = !SVGPlane || SVGPlaneName != SVGPlane->GetFName();
	if (bPlaneActorNeedsToBeCreated)
	{
		DestroySVGPlane();
		SVGPlane = NewObject<UStaticMeshComponent>(this, SVGPlaneName);
		AddInstanceComponent(SVGPlane);
	}

	SVGPlane->SetStaticMesh(PlaneStaticMesh);
	SVGPlane->AttachToComponent(RootComponent, FAttachmentTransformRules::SnapToTargetIncludingScale);
	SVGPlane->SetRelativeRotation(FRotator(0, 90, 90));

	RefreshPlaneMaterialInstance();

	SVGPlane->RegisterComponent();

	TriggerActorDetailsRefresh();
}

void ASVGActor::PostLoad()
{
	Super::PostLoad();

	if (bMeshesShouldBeGenerated)
	{
		Generate();
	}

	if (RenderMode == ESVGRenderMode::DynamicMesh3D)
	{
		RefreshAllShapesMaterials();
	}
}

void ASVGActor::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	RemoveAllDelegates();
}

void ASVGActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();
	Initialize();
}

void ASVGActor::Generate()
{
	switch (RenderMode)
	{
		case ESVGRenderMode::DynamicMesh3D:
			ScheduleShapesGeneration();
			break;

		case ESVGRenderMode::Texture2D:
#if WITH_EDITOR
			HideGenerationStartNotification();
#endif
			CreateSVGPlane();
			break;
	}
}

void ASVGActor::BeginDestroy()
{
	CleanupEverything();
	Super::BeginDestroy();
}

void ASVGActor::Destroyed()
{
	Super::Destroyed();

#if WITH_EDITOR
		HideGenerationStartNotification();
#endif
}

void ASVGActor::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FSVGActorVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FSVGActorVersion::GUID);

	if (Version < FSVGActorVersion::ShapeComponents)
	{
		ShapesOffset = 0.0f;
		bMeshesShouldBeGenerated = true;

		// Old shapes are still part of instance components, so we need to get rid of them
		bClearInstanceComponents = true;
	}
}

void ASVGActor::AddStrokeComponent(const TArray<FVector>& InPoints, float InThickness, const FColor& InColor, bool bIsClosed, bool bIsClockwise
                                   , float InExtrudeOffset /* = 0*/, const FString& InName /* = "" */)
{
	if (InPoints.Num() >= 2)
	{
		FString StrokeShapeName;

		if (!InName.IsEmpty())
		{
			StrokeShapeName = TEXT("Stroke_") + InName;
		}
		else
		{
			StrokeShapeName = TEXT("Stroke_NoName_") + FString::FromInt(UnnamedStrokesNum++);
		}

		const FName StrokeShapeUniqueName = MakeUniqueObjectName(this, UTexture2D::StaticClass(), FName(*StrokeShapeName));

		USVGStrokeComponent* DynStrokeComponent = NewObject<USVGStrokeComponent>(this, StrokeShapeUniqueName, RF_Transactional);
		AddInstanceComponent(DynStrokeComponent);
		DynStrokeComponent->RegisterComponent();
		DynStrokeComponent->AttachToComponent(StrokeShapesRoot, FAttachmentTransformRules::KeepRelativeTransform);
		
		ShapeComponents.Add(DynStrokeComponent);

		DynStrokeComponent->SetExtrudeType(ExtrudeType);

		FSVGStrokeParameters StrokeParams(InPoints);
		StrokeParams.Thickness = InThickness * StrokesWidth;
		StrokeParams.InColor = InColor;
		StrokeParams.bIsClosed = bIsClosed;
		StrokeParams.bIsClockwise = bIsClockwise;
		StrokeParams.JoinStyle = StrokeJoinStyle;
		StrokeParams.Extrude = InExtrudeOffset;
		StrokeParams.bUnlit = bSVGIsUnlit;
		StrokeParams.bCastShadow = bSVGCastsShadow;

		DynStrokeComponent->GenerateStrokeMesh(StrokeParams);

		// Shapes hidden while the whole set is being generated
		DynStrokeComponent->SetVisibility(false);
	}
}

void ASVGActor::AddFillComponent(const TArray<FSVGPathPolygon>& InShapesToDraw, const FColor& InColor, float InExtrudeOffset /* = 0 */, const FString& InName /* = "" */)
{
	if (!InShapesToDraw.IsEmpty())
	{
		FString FillShapeName;

		if (!InName.IsEmpty())
		{
			FillShapeName = TEXT("Fill_") + InName;
		}
		else
		{
			FillShapeName = TEXT("Fill_NoName_") + FString::FromInt(UnnamedFillsNum++);
		}

		const FName FillShapeUniqueName = MakeUniqueObjectName(this, UTexture2D::StaticClass(), FName(*FillShapeName));

		USVGFillComponent* DynFillComponent = NewObject<USVGFillComponent>(this, FillShapeUniqueName, RF_Transactional);
		AddInstanceComponent(DynFillComponent);
		DynFillComponent->RegisterComponent();
		DynFillComponent->AttachToComponent(FillShapesRoot, FAttachmentTransformRules::KeepRelativeTransform);
		
		ShapeComponents.Add(DynFillComponent);

		DynFillComponent->SetExtrudeType(ExtrudeType);

		FSVGFillParameters FillParams(InShapesToDraw);
		FillParams.Color = InColor;
		FillParams.Extrude = InExtrudeOffset;
		FillParams.bSimplify =  UE::SVGImporter::Private::bSimplifyFills;
		FillParams.BevelDistance = BevelDistance;
		FillParams.bSmoothShapes = bSmoothFillShapes;
		FillParams.SmoothingOffset = SmoothingOffset;
		FillParams.bUnlit = bSVGIsUnlit;
		FillParams.bCastShadow = bSVGCastsShadow;

		DynFillComponent->GenerateFillMesh(FillParams);

		// Shapes hidden while the whole set is being generated
		DynFillComponent->SetVisibility(false);
	}
}

void ASVGActor::UpdateFillShapesSmoothing()
{
	ScheduleFillsSmoothUpdate();
}

void ASVGActor::UpdateFillShapesSmoothingEnable()
{
	static constexpr bool bSmoothingEnabledFlagHasChanged = true;
	ScheduleFillsSmoothUpdate(bSmoothingEnabledFlagHasChanged);
}

void ASVGActor::UpdateStrokesVisibility()
{
	for (USVGStrokeComponent* StrokeComponent : GetStrokeComponents())
	{
		if (StrokeComponent)
		{
			StrokeComponent->SetVisibility(!bIgnoreStrokes);
		}
	}
}

void ASVGActor::UpdateStrokesWidth()
{
	if (!bIgnoreStrokes)
	{
		ScheduleStrokesWidthUpdate();
	}
}

void ASVGActor::CreateMeshesFromShape(const FSVGShape& InShape)
{
	if (!SVGData)
	{
#if WITH_EDITOR
		DisplayMissingSVGDataError(TEXT("Cannot generate SVG geometry."));
#endif
		return;
	}

	const int ExtrusionSteps = SVGData->Shapes.IsEmpty() ? 1 : SVGData->Shapes.Num();
	const float DepthExtrude = FMath::Min( UE::SVGImporter::Private::ExtrudeDepthIncrement, UE::SVGImporter::Private::MaxExtrudeDepthTotal / ExtrusionSteps);

	if (FMath::IsNearlyZero(CurrExtrudeForDepth))
	{
		CurrExtrudeForDepth += DepthExtrude;
	}

	// Fill shapes need some information other than just vertices in order to be properly generated
	TArray<FSVGPathPolygon> FillShapesToRender;
	TArray<TArray<FVector>> StrokesToRender;

	for (const FSVGPathPolygon& ShapeToCheck : InShape.GetPolygons())
	{
		if (InShape.HasStroke())
		{
			StrokesToRender.Add(ShapeToCheck.GetVertices());
		}

		bool bShouldDrawFill = true;

		if (InShape.PolygonsNum() == 1)
		{
			bShouldDrawFill = InShape.HasFill();
		}

		if (bShouldDrawFill)
		{
			FillShapesToRender.Add(ShapeToCheck);
		}
	}

	bool bIncreaseExtrudeForDepth = false;

	if (InShape.HasFill())
	{
		AddFillComponent(FillShapesToRender, InShape.GetFillColor(), CurrExtrudeForDepth, InShape.GetId());
		bIncreaseExtrudeForDepth = true;
	}

	if (!bIgnoreStrokes)
	{
		const float StrokeComponentExtrude = CurrExtrudeForDepth * UE::SVGImporter::Private::StrokeAddedDepthMultiplier;

		int32 AddedStrokes = 0;
		for (const TArray<FVector>& StrokePoints : StrokesToRender)
		{
			if (InShape.HasStroke())
			{
				FString StrokeName;
				if (StrokesToRender.Num() > 1)
				{
					StrokeName = InShape.GetId() + TEXT("_") + FString::FromInt(AddedStrokes++);
				}
				else
				{
					StrokeName = InShape.GetId();
				}

				AddStrokeComponent(StrokePoints, InShape.GetStyle().GetStrokeWidth(), InShape.GetStrokeColor(), InShape.IsClosed(), InShape.IsClockwise(), StrokeComponentExtrude, StrokeName);

				bIncreaseExtrudeForDepth = true;
			}
		}
	}

	if (bIncreaseExtrudeForDepth)
	{
		CurrExtrudeForDepth += DepthExtrude;
	}
}

void ASVGActor::UpdateFillShapesExtrude(float InExtrude, ESVGEditMode InEditMode)
{
	for (USVGFillComponent* FillComponent : GetFillComponents())
	{
		if (FillComponent)
		{
			// set the minimum extrude value, the same for all shapes
			FillComponent->SetExtrudeType(ExtrudeType);
			FillComponent->SetMeshEditMode(InEditMode);
			FillComponent->SetMinExtrudeValue(InExtrude);
		}
	}
}

void ASVGActor::UpdateStrokeShapesExtrude(float InExtrude, ESVGEditMode InEditMode)
{
	for (USVGStrokeComponent* StrokeComponent : GetStrokeComponents())
	{
		if (StrokeComponent)
		{
			// set the minimum extrude value, the same for all shapes
			StrokeComponent->SetExtrudeType(ExtrudeType);
			StrokeComponent->SetMeshEditMode(InEditMode);
			StrokeComponent->SetMinExtrudeValue(InExtrude);
			StrokeComponent->SetJointStyle(StrokeJoinStyle);
		}
	}
}

void ASVGActor::UpdateFillShapesBevel(float InBevel, ESVGEditMode InEditMode)
{
	if (InEditMode == ESVGEditMode::Interactive)
	{
		for (USVGFillComponent* FillComponent : GetFillComponents())
		{
			if (FillComponent)
			{
				// set the minimum extrude value, the same for all shapes
				FillComponent->SetMeshEditMode(InEditMode);
				FillComponent->SetBevel(InBevel);
			}
		}
	}
	else if (InEditMode == ESVGEditMode::ValueSet)
	{
		ScheduleBevelsUpdate();
	}
}

void ASVGActor::DestroySVGPlane()
{
	if (SVGPlane)
	{
		SVGPlane->DestroyComponent();
		SVGPlane = nullptr;

		PlaneMaterialInstance = nullptr;
	}
}

void ASVGActor::UpdateFillsSmoothingOnEndFrame()
{
	static constexpr double BudgetedTime = 1 / 30.f; // 30 frames
	double CurrentBudgetUsed = 0.f;

	while (CurrentBudgetUsed < BudgetedTime)
	{
		const double PreProcessTime = FPlatformTime::Seconds();

		const TArray<TObjectPtr<USVGFillComponent>> FillComponents = GetFillComponents();
		for (int i = 0; i < UE::SVGImporter::Private::GeneratedMeshesPerLoop; i++)
		{
			if (UpdatedFillsCount < FillComponents.Num())
			{
				if (const TObjectPtr<USVGFillComponent>& FillComponent = FillComponents[UpdatedFillsCount])
				{
					const float NewSmoothOffset = bSmoothFillShapes ? SmoothingOffset : UE::SVGImporter::Private::DefaultSmoothOffset;
					FillComponent->SetSmoothFillShapes(bSmoothFillShapes, NewSmoothOffset);
					UpdatedFillsCount++;
				}
			}
			else
			{
				UpdatedFillsCount = 0;
				CenterMeshes();
#if WITH_EDITOR
				FillsSmoothingTransaction.Reset();
#endif
				FCoreDelegates::OnEndFrame.Remove(FillsUpdateDelegateHandle);

				OnFillsUpdateEnd_Delegate.Broadcast();
				OnFillsUpdateEnd_Delegate.RemoveAll(this);

				break;
			}
		}

		const double DeltaTime = FPlatformTime::Seconds() - PreProcessTime;
		CurrentBudgetUsed += DeltaTime;
	}
}

void ASVGActor::UpdateStrokesWidthOnEndFrame()
{
	static constexpr double BudgetedTime = 1 / 30.f; // 30 frames
	double CurrentBudgetUsed = 0.f;

	while (CurrentBudgetUsed < BudgetedTime)
	{
		const double PreProcessTime = FPlatformTime::Seconds();

		const TArray<TObjectPtr<USVGStrokeComponent>> StrokeComponents = GetStrokeComponents();
		for (int i = 0; i < UE::SVGImporter::Private::GeneratedMeshesPerLoop; i++)
		{
			if (UpdatedStrokesCount < StrokeComponents.Num())
			{
				if (const TObjectPtr<USVGStrokeComponent>& StrokeComponent = StrokeComponents[UpdatedStrokesCount])
				{
					StrokeComponent->SetStrokeWidth(StrokesWidth);
					UpdatedStrokesCount++;
				}
			}
			else
			{
				UpdatedStrokesCount = 0;
				CenterMeshes();
#if WITH_EDITOR
				StrokesWidthTransaction.Reset();
#endif
				FCoreDelegates::OnEndFrame.Remove(StrokesUpdateDelegateHandle);
				OnStrokesUpdateEnd_Delegate.Broadcast();
				OnStrokesUpdateEnd_Delegate.RemoveAll(this);

				break;
			}
		}

		const double DeltaTime = FPlatformTime::Seconds() - PreProcessTime;
		CurrentBudgetUsed += DeltaTime;
	}
}

bool ASVGActor::CanUpdateShapes() const
{
	return !bIsGeneratingMeshes;
}

bool ASVGActor::CanGenerateShapes() const
{
	return bMeshesShouldBeGenerated && !bIsGeneratingMeshes;
}

void ASVGActor::RemoveAllDelegates() const
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}

void ASVGActor::ScheduleShapesGeneration()
{
	if (!bMeshesShouldBeGenerated)
	{
		return;
	}

	if (!SVGData)
	{
#if WITH_EDITOR
		DisplayMissingSVGDataError(TEXT("Cannot generate SVG geometry."));
#endif

		DestroySVGDynMeshComponents();
		DestroySVGPlane();
		return;
	}

	UnnamedFillsNum = 0;
	UnnamedStrokesNum = 0;

	DestroySVGPlane();

	if (CanGenerateShapes())
	{
		SVGName = SVGData->GetName();

		FCoreDelegates::OnEndFrame.Remove(GenMeshDelegateHandle);
		FCoreDelegates::OnEndFrame.Remove(FillsUpdateDelegateHandle);
		FCoreDelegates::OnEndFrame.Remove(StrokesUpdateDelegateHandle);

		ResetShapes();

		HideShapes();

		CurrExtrudeForDepth = 0.0f;
		GeneratedShapesCount = 0;

		GenMeshDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::GenerateShapesOnEndFrame);
		OnShapesGenerationEnd_Delegate.AddUObject(this, &ASVGActor::OnShapesGenerationEnd);
	}
}

void ASVGActor::ScheduleFillExtrudeUpdate()
{
	bFillsExtrudeFinished = false;
	FCoreDelegates::OnEndFrame.Remove(FillsUpdateDelegateHandle);
	UpdatedFillsCount = 0;

	FillsUpdateDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::UpdateFillsExtrudeOnEndFrame);
	OnFillsExtrudeEnd_Delegate.AddUObject(this, &ASVGActor::OnShapesExtrudeEnd);
}

void ASVGActor::ScheduleStrokesExtrudeUpdate()
{
	bStrokesExtrudeFinished = false;
	FCoreDelegates::OnEndFrame.Remove(StrokesUpdateDelegateHandle);	
	UpdatedStrokesCount = 0;

	StrokesUpdateDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::UpdateStrokesExtrudeOnEndFrame);
	OnStrokesExtrudeEnd_Delegate.AddUObject(this, &ASVGActor::OnShapesExtrudeEnd);
}

void ASVGActor::ScheduleBevelsUpdate()
{
	FCoreDelegates::OnEndFrame.Remove(BevelUpdateDelegateHandle);

#if WITH_EDITOR
	BevelsUpdateTransaction.Reset();
	BevelsUpdateTransaction = MakeShared<FScopedTransaction>(LOCTEXT("FillShapesBevelUpdate", "Changed bevel on SVG fill elements"));
	this->Modify();
#endif

	UpdatedBevelsCount = 0;

	BevelUpdateDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::UpdateBevelsOnEndFrame);
}

void ASVGActor::ScheduleFillsSmoothUpdate(bool bInHasSmoothingFlagChanged)
{
	if (CanUpdateShapes())
	{
		FCoreDelegates::OnEndFrame.Remove(FillsUpdateDelegateHandle);

#if WITH_EDITOR
		FText SessionName;
		if (bInHasSmoothingFlagChanged)
		{
			SessionName = LOCTEXT("FillsSmoothFlagChanged", "Changed smoothing enabled state for SVG fill elements");
		}
		else
		{
			SessionName = LOCTEXT("FillsSmoothUpdate", "Changed smoothing on SVG fill elements");
		}

		FillsSmoothingTransaction.Reset();
		FillsSmoothingTransaction = MakeShared<FScopedTransaction>(SessionName);
		this->Modify();
#endif

		UpdatedFillsCount = 0;

		FillsUpdateDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::UpdateFillsSmoothingOnEndFrame);
	}
}

void ASVGActor::ScheduleStrokesWidthUpdate()
{
	if (CanUpdateShapes())
	{
		FCoreDelegates::OnEndFrame.Remove(StrokesUpdateDelegateHandle);

#if WITH_EDITOR
		StrokesWidthTransaction.Reset();
		StrokesWidthTransaction = MakeShared<FScopedTransaction>(LOCTEXT("StrokesWidthUpdate", "Changed width on SVG stroke elements"));
		this->Modify();
#endif

		UpdatedStrokesCount = 0;

		StrokesUpdateDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &ASVGActor::UpdateStrokesWidthOnEndFrame);
	}
}


void ASVGActor::TriggerActorDetailsRefresh()
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.BroadcastComponentsEdited();
#endif

	if (USVGEngineSubsystem* SVGWorldSubsystem = USVGEngineSubsystem::Get())
	{
		SVGWorldSubsystem->GetSVGActorComponentsReadyDelegate().ExecuteIfBound(this);
	}
}

void ASVGActor::GenerateNextMesh()
{
	if (!SVGData)
	{
		FCoreDelegates::OnEndFrame.Remove(GenMeshDelegateHandle);

#if WITH_EDITOR
		DisplayMissingSVGDataError(TEXT("Cannot generate SVG geometry."));
#endif
		return;
	}

	if (SVGData->Shapes.IsValidIndex(GeneratedShapesCount))
	{
#if WITH_EDITOR
		if (GeneratedShapesCount == 0)
		{
			ShowGenerationStartNotification();
		}
#endif

		const FSVGShape& Shape = SVGData->Shapes[GeneratedShapesCount];
		CreateMeshesFromShape(Shape);
		GeneratedShapesCount++;
	}
}

void ASVGActor::CenterMeshes()
{
	if (FillShapesRoot)
	{
		FillShapesRoot->SetRelativeLocation(FVector::ZeroVector);
	}

	if (StrokeShapesRoot)
	{
		StrokeShapesRoot->SetRelativeLocation(FVector::ZeroVector);
	}

	FVector Origin;
	FVector Extent;
	GetActorBounds(false, Origin, Extent);

	Origin -= GetActorLocation();

	if (ExtrudeType == ESVGExtrudeType::FrontFaceOnly)
	{
		Origin.X += Extent.X;
	}

	OffsetToCenterSVGMeshes = -Origin / GetActorScale3D();

	OffsetToCenterSVGMeshes.X = 0.0f;

	if (FillShapesRoot)
	{
		FillShapesRoot->SetRelativeLocation(OffsetToCenterSVGMeshes);
	}

	if (StrokeShapesRoot)
	{
		StrokeShapesRoot->SetRelativeLocation(OffsetToCenterSVGMeshes);
	}
}

void ASVGActor::ComputeOffsetScale()
{
	FVector Origin;
	FVector Extent;
	GetActorBounds(false, Origin, Extent);
	ShapesOffsetScale = FMath::Max(Extent.Y, Extent.Z) * UE::SVGImporter::Private::OffsetScaleMultiplier;
}

void ASVGActor::ApplyOffset()
{
	float CurrentOffset = 0.0f;
	const float ScaledOffset = ShapesOffset * ShapesOffsetScale * Scale;

	for (const TObjectPtr<USVGDynamicMeshComponent>& ShapeComponent : ShapeComponents)
	{
		if (ShapeComponent)
		{
			const FVector CurrentLocation = ShapeComponent->GetRelativeLocation();
			const FVector Offset = FVector(-CurrentOffset, CurrentLocation.Y, CurrentLocation.Z);
			ShapeComponent->SetRelativeLocation(Offset);
			CurrentOffset += ScaledOffset;
		}
	}
}

void ASVGActor::ApplyScaleAndCenter()
{
	for (USVGDynamicMeshComponent* ShapeComponent : ShapeComponents)
	{
		if (ShapeComponent)
		{
			ShapeComponent->ScaleShape(Scale);
		}
	}

	CenterMeshes();
}

void ASVGActor::OnVisibilityChanged()
{
	if (bVisible)
	{
		ShowShapes();
	}
	else
	{
		HideShapes();
	}
}

void ASVGActor::GenerateShapesOnEndFrame()
{
	if (bIsGeneratingMeshes || !bMeshesShouldBeGenerated)
	{
		return;
	}

	TGuardValue<bool> GeneratingGuard(bIsGeneratingMeshes, true);

	if (!SVGData)
	{
		FCoreDelegates::OnEndFrame.Remove(GenMeshDelegateHandle);

#if WITH_EDITOR
		DisplayMissingSVGDataError(TEXT("Cannot generate SVG geometry."));
#endif
		return;
	}

	static constexpr double BudgetedTime = 1 / 30.f; //30 frames
	double CurrentBudgetUsed = 0.f;

	bool bHasFinished = false;

	while (CurrentBudgetUsed < BudgetedTime && !bHasFinished)
	{
		const double PreProcessTime = FPlatformTime::Seconds();

		for (int i = 0; i < UE::SVGImporter::Private::GeneratedMeshesPerLoop; i++)
		{
			GenerateNextMesh();

			// If no more shapes have to be created, unregister callback and finish things up
			if (!SVGData->Shapes.IsValidIndex(GeneratedShapesCount))
			{
				bHasFinished = true;
				bMeshesShouldBeGenerated = false;
				FCoreDelegates::OnEndFrame.Remove(GenMeshDelegateHandle);

				ScheduleFillExtrudeUpdate();
                ScheduleStrokesExtrudeUpdate();

				// notify details panel, so that newly added components are shown right away
				TriggerActorDetailsRefresh();

				bSVGHasFills = !GetFillComponents().IsEmpty();
				bSVGHasStrokes = !GetStrokeComponents().IsEmpty();

#if WITH_EDITOR
				GenMeshTransaction.Reset();
#endif

				ApplyScaleAndCenter();
				ComputeOffsetScale();
				ApplyOffset();

				break;
			}
		}

		const double DeltaTime = FPlatformTime::Seconds() - PreProcessTime;
		CurrentBudgetUsed += DeltaTime;
	}
}

void ASVGActor::ShowShapes() const
{
	if (RootComponent)
	{
		RootComponent->SetVisibility(bVisible, true);
	}
}

void ASVGActor::HideShapes() const
{
	if (RootComponent)
	{
		RootComponent->SetVisibility(false, true);
	}
}

void ASVGActor::OnShapesGenerationEnd()
{
#if WITH_EDITOR
	ShowGenerationFinishedNotification();
	RerunConstructionScripts();
#endif

	ShowShapes();

	// Make sure shapes are updated after generation
	RefreshFillShapesBevel(EPropertyChangeType::ValueSet);
	RefreshFillShapesExtrude(EPropertyChangeType::ValueSet);
	RefreshStrokeShapesExtrude(EPropertyChangeType::ValueSet);
	ApplyOffset();

	if (RenderMode == ESVGRenderMode::DynamicMesh3D)
	{
		RefreshAllShapesMaterials();
	}
}

void ASVGActor::OnShapesExtrudeEnd()
{
	if (bFillsExtrudeFinished && bStrokesExtrudeFinished)
	{
		OnShapesGenerationEnd_Delegate.Broadcast();
		OnShapesGenerationEnd_Delegate.RemoveAll(this);
	}
}

void ASVGActor::UpdateFillsExtrudeOnEndFrame()
{
	static constexpr double BudgetedTime = 1 / 30.f; // 30 frames
	double CurrentBudgetUsed = 0.f;

	while (CurrentBudgetUsed < BudgetedTime)
	{
		const double PreProcessTime = FPlatformTime::Seconds();

		const TArray<TObjectPtr<USVGFillComponent>> FillComponents = GetFillComponents();
		for (int i = 0; i < UE::SVGImporter::Private::GeneratedMeshesPerLoop; i++)
		{
			if (UpdatedFillsCount < FillComponents.Num())
			{
				if (const TObjectPtr<USVGFillComponent>& FillComponent = FillComponents[UpdatedFillsCount])
				{
					// set the minimum extrude value, the same for all shapes
					FillComponent->SetExtrudeType(ExtrudeType);
					FillComponent->SetMeshEditMode(ESVGEditMode::ValueSet);
					FillComponent->SetMinExtrudeValue(FillsExtrude);
					UpdatedFillsCount++;
				}
			}
			else
			{
				UpdatedFillsCount = 0;

				CenterMeshes();

				FCoreDelegates::OnEndFrame.Remove(FillsUpdateDelegateHandle);
				bFillsExtrudeFinished = true;
				OnFillsExtrudeEnd_Delegate.Broadcast();
				OnFillsExtrudeEnd_Delegate.RemoveAll(this);

				break;
			}
		}

		const double DeltaTime = FPlatformTime::Seconds() - PreProcessTime;
		CurrentBudgetUsed += DeltaTime;
	}
}

void ASVGActor::UpdateStrokesExtrudeOnEndFrame()
{
	static constexpr double BudgetedTime = 1 / 30.f; // 30 frames
	double CurrentBudgetUsed = 0.f;

	while (CurrentBudgetUsed < BudgetedTime)
	{
		const double PreProcessTime = FPlatformTime::Seconds();

		const TArray<TObjectPtr<USVGStrokeComponent>> StrokeComponents = GetStrokeComponents();
		for (int i = 0; i < UE::SVGImporter::Private::GeneratedMeshesPerLoop; i++)
		{
			if (UpdatedStrokesCount >= StrokeComponents.Num())
			{
				UpdatedStrokesCount = 0;
				CenterMeshes();

				FCoreDelegates::OnEndFrame.Remove(StrokesUpdateDelegateHandle);
				bStrokesExtrudeFinished = true;
				OnStrokesExtrudeEnd_Delegate.Broadcast();
				OnStrokesExtrudeEnd_Delegate.RemoveAll(this);

				break;
			}
			else if (const TObjectPtr<USVGStrokeComponent>& StrokeComponent = StrokeComponents[UpdatedStrokesCount])
			{
				// set the minimum extrude value, the same for all shapes
				StrokeComponent->SetExtrudeType(ExtrudeType);
				StrokeComponent->SetMeshEditMode(ESVGEditMode::ValueSet);
				StrokeComponent->SetMinExtrudeValue(StrokesExtrude);
				StrokeComponent->SetJointStyle(StrokeJoinStyle);
				UpdatedStrokesCount++;
			}
		}

		const double DeltaTime = FPlatformTime::Seconds() - PreProcessTime;
		CurrentBudgetUsed += DeltaTime;
	}
}

void ASVGActor::UpdateBevelsOnEndFrame()
{
	const TArray<TObjectPtr<USVGFillComponent>> FillComponents = GetFillComponents();
	for (int i = 0; i < UE::SVGImporter::Private::BevelUpdatesPerFrame; i++)
	{
		if (UpdatedBevelsCount >= FillComponents.Num())
		{
			UpdatedBevelsCount = 0;
#if WITH_EDITOR
			BevelsUpdateTransaction.Reset();
#endif
			FCoreDelegates::OnEndFrame.Remove(BevelUpdateDelegateHandle);
			ApplyOffset();
			break;
		}
		else if (const TObjectPtr<USVGFillComponent>& FillComponent = FillComponents[UpdatedBevelsCount])
		{
			// set the minimum extrude value, the same for all shapes
			FillComponent->SetMeshEditMode(ESVGEditMode::ValueSet);
			FillComponent->SetBevel(BevelDistance);
			UpdatedBevelsCount++;
		}
	}
}


void ASVGActor::SetScale(float InScale)
{
	if (!FMath::IsNearlyEqual(Scale, InScale))
	{
		Scale = InScale;

		ApplyScaleAndCenter();
	}
}

void ASVGActor::SetShapesOffset(float InShapesOffset)
{
	if (ShapesOffset != InShapesOffset)
	{
		ShapesOffset = InShapesOffset;

		ApplyOffset();
	}
}

void ASVGActor::SetFillsExtrude(float InFillsExtrude)
{
#if WITH_EDITOR
	if (FillsExtrude != InFillsExtrude || bFillsExtrudeInteractiveUpdate)
#else
	if (FillsExtrude != InFillsExtrude)
#endif
	{
		FillsExtrude = InFillsExtrude;

		RefreshFillShapesExtrude();
		ApplyOffset();

#if WITH_EDITOR
		bFillsExtrudeInteractiveUpdate = false;
#endif
	}
}

void ASVGActor::SetStrokesExtrude(float InStrokesExtrude)
{
#if WITH_EDITOR
	if (StrokesExtrude != InStrokesExtrude || bStrokesExtrudeInteractiveUpdate)
#else
	if (StrokesExtrude != InStrokesExtrude)
#endif
	{
		StrokesExtrude = InStrokesExtrude;

		RefreshStrokeShapesExtrude();
		ApplyOffset();

#if WITH_EDITOR
		bStrokesExtrudeInteractiveUpdate = false;
#endif
	}
}

void ASVGActor::SetBevelDistance(float InBevelDistance)
{
	if (BevelDistance != InBevelDistance)
	{
		BevelDistance = InBevelDistance;

		RefreshFillShapesBevel();
		ApplyOffset();
	}
}

void ASVGActor::SetStrokesWidth(float InStrokesWidth)
{
	if (StrokesWidth != InStrokesWidth)
	{
		StrokesWidth = InStrokesWidth;

		UpdateStrokesWidth();
	}
}

void ASVGActor::SetVisible(bool bInVisible)
{
	if (bVisible == bInVisible)
	{
		return;
	}

	bVisible = bInVisible;
	OnVisibilityChanged();
}

void ASVGActor::RefreshFillShapesExtrude(const EPropertyChangeType::Type ChangeType)
{
	const float NewExtrudeValue = ExtrudeType == ESVGExtrudeType::None ? 0.0f : FillsExtrude;

	if (ChangeType == EPropertyChangeType::Interactive)
	{
		UpdateFillShapesExtrude(NewExtrudeValue, ESVGEditMode::Interactive);
	}
	else if (ChangeType == EPropertyChangeType::ValueSet)
	{
		UpdateFillShapesExtrude(NewExtrudeValue, ESVGEditMode::ValueSet);
	}
}

void ASVGActor::RefreshStrokeShapesExtrude(const EPropertyChangeType::Type ChangeType)
{
	const float NewExtrudeValue = ExtrudeType == ESVGExtrudeType::None ? 0.0f : StrokesExtrude;

	if (ChangeType == EPropertyChangeType::Interactive)
	{
		UpdateStrokeShapesExtrude(NewExtrudeValue, ESVGEditMode::Interactive);
	}
	else if (ChangeType == EPropertyChangeType::ValueSet)
	{
		UpdateStrokeShapesExtrude(NewExtrudeValue, ESVGEditMode::ValueSet);
	}
}

void ASVGActor::RefreshFillShapesBevel(const EPropertyChangeType::Type ChangeType)
{
	if (ChangeType == EPropertyChangeType::Interactive)
	{
		UpdateFillShapesBevel(BevelDistance, ESVGEditMode::Interactive);
	}
	else if (ChangeType == EPropertyChangeType::ValueSet)
	{
		UpdateFillShapesBevel(BevelDistance, ESVGEditMode::ValueSet);
	}
}

void ASVGActor::RefreshAllShapes()
{
	bMeshesShouldBeGenerated = true;
	ScheduleShapesGeneration();
}

void ASVGActor::RefreshAllShapesMaterials()
{
	int32 SortPriority = 0;
	for (USVGDynamicMeshComponent* ShapeComponent : ShapeComponents)
	{
		if (ShapeComponent)
		{
			ShapeComponent->SetIsUnlit(bSVGIsUnlit);
			ShapeComponent->SetTranslucentSortPriority(SortPriority++);
		}
	}
}

void ASVGActor::RefreshAllShapesCastShadow()
{
	for (USVGDynamicMeshComponent* ShapeComponent : ShapeComponents)
	{
		if (ShapeComponent)
		{
			ShapeComponent->SetCastShadow(bSVGCastsShadow);
		}
	}
}

void ASVGActor::RefreshSVGPlaneCastShadow() const
{
	if (SVGPlane)
	{
		SVGPlane->SetCastShadow(bSVGCastsShadow);
	}
}

TArray<UDynamicMeshComponent*> ASVGActor::GetSVGDynamicMeshes()
{
	TArray<UDynamicMeshComponent*> OutMeshComponents;

	for (UDynamicMeshComponent* ShapeComponent : ShapeComponents)
	{
		OutMeshComponents.Add(ShapeComponent);
	}

	return OutMeshComponents;
}

TArray<TObjectPtr<USVGFillComponent>> ASVGActor::GetFillComponents() const
{
	TArray<TObjectPtr<USVGFillComponent>> FillComponents;

	for (const TObjectPtr<USVGDynamicMeshComponent>& ShapeComponent : ShapeComponents)
	{
		if (USVGFillComponent* FillComponent = Cast<USVGFillComponent>(ShapeComponent))
		{
			FillComponents.Add(FillComponent);
		}
	}

	return FillComponents;
}

TArray<TObjectPtr<USVGStrokeComponent>> ASVGActor::GetStrokeComponents() const
{
	TArray<TObjectPtr<USVGStrokeComponent>> StrokeComponents;
	StrokeComponents.Reserve(ShapeComponents.Num());

	for (const TObjectPtr<USVGDynamicMeshComponent>& ShapeComponent : ShapeComponents)
	{
		if (USVGStrokeComponent* StrokeComponent = Cast<USVGStrokeComponent>(ShapeComponent))
		{
			StrokeComponents.Add(StrokeComponent);
		}
	}

	return StrokeComponents;
}

#if WITH_EDITOR
void ASVGActor::SetFillsExtrudeInteractive(float InFillsExtrude)
{
	if (FillsExtrude != InFillsExtrude)
	{
		FillsExtrude = InFillsExtrude;

		RefreshFillShapesExtrude(EPropertyChangeType::Interactive);
		ApplyOffset();

		bFillsExtrudeInteractiveUpdate = true;
	}
}

void ASVGActor::SetStrokesExtrudeInteractive(float InStrokesExtrude)
{
	if (StrokesExtrude != InStrokesExtrude)
	{
		StrokesExtrude = InStrokesExtrude;

		RefreshStrokeShapesExtrude(EPropertyChangeType::Interactive);
		ApplyOffset();

		bStrokesExtrudeInteractiveUpdate = true;
	}
}

void ASVGActor::BakeToBlueprint() const
{
	if (RenderMode == ESVGRenderMode::DynamicMesh3D)
	{
		if (!bIsGeneratingMeshes)
		{
			FString SavePath;

			constexpr const TCHAR* BakedSuffix = TEXT("_Baked");

			if (SVGData)
			{
				SavePath = SVGData->GetPackage()->GetPathName() + BakedSuffix;
			}
			else
			{
				// at this stage, we can still try to bake the actor, even if the SVGData is gone, because the geometry has already been generated
				SavePath = TEXT("/Game/Baked_SVGs/") + SVGName + BakedSuffix;

				FNotificationInfo Info(FText::Format(LOCTEXT("SVGDataMissingWhenBaking", "SVGData asset was missing. Baked actor for SVG {0} will be generated inside Content/Baked_SVGs folder."), FText::FromString(SVGName)));
				Info.bFireAndForget = true;
				Info.bUseLargeFont = true;
				Info.ExpireDuration = UE::SVGImporter::Private::DefaultNotificationsExpireTime;
				Info.FadeOutDuration = UE::SVGImporter::Private::DefaultNotificationsFadeOutTime;

				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_None);
			}

			FSVGImporterUtils::BakeSVGActorToBlueprint(this, SavePath);
		}
		else
		{
			FNotificationInfo Info(FText::Format(LOCTEXT("SVGStillGenerating", "Dynamic meshes for SVG {0} are being generated. Cannot bake to actor yet."), FText::FromString(SVGName)));
			Info.bFireAndForget = true;
			Info.bUseLargeFont = true;
			Info.ExpireDuration = UE::SVGImporter::Private::DefaultNotificationsExpireTime;
			Info.FadeOutDuration = UE::SVGImporter::Private::DefaultNotificationsFadeOutTime;

			FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
	else
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("SVGBakeNotAvailableFor2DRender", "{0} is currently rendered as a texture. Baking to SVG Actor not available. Switch to 3D render mode."), FText::FromString(SVGName)));
		Info.bFireAndForget = true;
		Info.bUseLargeFont = true;
		Info.ExpireDuration = UE::SVGImporter::Private::DefaultNotificationsExpireTime;
		Info.FadeOutDuration = UE::SVGImporter::Private::DefaultNotificationsFadeOutTime;

		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

void ASVGActor::DisplayMissingSVGDataError(const FString& InErrorMsg)
{
	if (bIsEditorPreviewActor)
	{
		return;
	}

	// in case this notification is still hanging around, let's remove it
	HideGenerationStartNotification();

	FNotificationInfo Info(FText::Format(LOCTEXT("SVGDataMissing", "SVGData asset is missing. {0}"), FText::FromString(InErrorMsg)));
	Info.bFireAndForget = true;
	Info.bUseLargeFont = true;
	Info.ExpireDuration = UE::SVGImporter::Private::DefaultNotificationsExpireTime;
	Info.FadeOutDuration = UE::SVGImporter::Private::DefaultNotificationsFadeOutTime;

	FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
}

void ASVGActor::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	static const FName FillsExtrudeName = GET_MEMBER_NAME_CHECKED(ASVGActor, FillsExtrude);
	static const FName StrokesExtrudeName = GET_MEMBER_NAME_CHECKED(ASVGActor, StrokesExtrude);
	static const FName BevelDistanceName = GET_MEMBER_NAME_CHECKED(ASVGActor, BevelDistance);
	static const FName ExtrudeTypeName = GET_MEMBER_NAME_CHECKED(ASVGActor, ExtrudeType);
	static const FName ShapesOffsetName = GET_MEMBER_NAME_CHECKED(ASVGActor, ShapesOffset);
	static const FName StrokeJoinStyleName = GET_MEMBER_NAME_CHECKED(ASVGActor, StrokeJoinStyle);
	static const FName RenderModeName = GET_MEMBER_NAME_CHECKED(ASVGActor, RenderMode);
	static const FName SVGDataName = GET_MEMBER_NAME_CHECKED(ASVGActor, SVGData);
	static const FName SmoothFillShapesName = GET_MEMBER_NAME_CHECKED(ASVGActor, bSmoothFillShapes);
	static const FName SmoothingOffsetName = GET_MEMBER_NAME_CHECKED(ASVGActor, SmoothingOffset);
	static const FName IgnoreStrokesName = GET_MEMBER_NAME_CHECKED(ASVGActor, bIgnoreStrokes);
	static const FName StrokesWidthName = GET_MEMBER_NAME_CHECKED(ASVGActor, StrokesWidth);
	static const FName SVGIsUnlitName = GET_MEMBER_NAME_CHECKED(ASVGActor, bSVGIsUnlit);
	static const FName SVGCastsShadowName = GET_MEMBER_NAME_CHECKED(ASVGActor, bSVGCastsShadow);
	static const FName ScaleName = GET_MEMBER_NAME_CHECKED(ASVGActor, Scale);
	static const FName VisibleName = GET_MEMBER_NAME_CHECKED(ASVGActor, bVisible);

	if (MemberName == FillsExtrudeName)
	{
		RefreshFillShapesExtrude(InPropertyChangedEvent.ChangeType);
		ApplyOffset();
	}
	else if (MemberName == StrokesExtrudeName)
	{
		RefreshStrokeShapesExtrude(InPropertyChangedEvent.ChangeType);
		ApplyOffset();
	}
	else if (MemberName == BevelDistanceName)
	{
		RefreshFillShapesBevel(InPropertyChangedEvent.ChangeType);
		ApplyOffset();
	}
	else if (MemberName == ExtrudeTypeName)
	{
		RefreshFillShapesExtrude(EPropertyChangeType::ValueSet);
		RefreshStrokeShapesExtrude(EPropertyChangeType::ValueSet);
		ApplyScaleAndCenter();
		ApplyOffset();
	}
	else if (MemberName == ShapesOffsetName)
	{
		ApplyOffset();
	}
	else if (MemberName == ScaleName)
	{
		ApplyScaleAndCenter();
		ApplyOffset();
	}
	else if (MemberName == StrokeJoinStyleName)
	{
		RefreshStrokeShapesExtrude(EPropertyChangeType::ValueSet);
		ApplyOffset();
	}
	else if (MemberName == RenderModeName)
	{
		if (RenderMode == ESVGRenderMode::DynamicMesh3D)
		{
			bMeshesShouldBeGenerated = true;
		}

		Generate();
	}
	else if (MemberName == SVGDataName)
	{
		if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			if (SVGData)
			{
				RefreshAllShapes();
			}
			else
			{
				DestroySVGDynMeshComponents();
				DestroySVGPlane();
			}
		}
	}
	else if (MemberName == SmoothFillShapesName)
	{
		UpdateFillShapesSmoothingEnable();
	}
	else if (MemberName == SmoothingOffsetName)
	{
		UpdateFillShapesSmoothing();
	}
	else if (MemberName == IgnoreStrokesName)
	{
		UpdateStrokesVisibility();
	}
	else if (MemberName == StrokesWidthName)
	{
		UpdateStrokesWidth();
	}
	else if (MemberName == SVGIsUnlitName)
	{
		if (RenderMode == ESVGRenderMode::DynamicMesh3D)
		{
			RefreshAllShapesMaterials();
		}
		else if (RenderMode == ESVGRenderMode::Texture2D)
		{
			RefreshPlaneMaterialInstance();
		}
	}
	else if (MemberName == SVGCastsShadowName)
	{
		if (RenderMode == ESVGRenderMode::DynamicMesh3D)
		{
			RefreshAllShapesCastShadow();
		}
		else if (RenderMode == ESVGRenderMode::Texture2D)
		{
			RefreshSVGPlaneCastShadow();
		}
	}
	else if (MemberName == VisibleName)
	{
		OnVisibilityChanged();
	}
}

void ASVGActor::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.HasPropertyChanges() && InTransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		const TArray<FName>& ChangedProperties = InTransactionEvent.GetChangedProperties();

		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(ASVGActor, FillsExtrude)))
		{
			RefreshFillShapesExtrude();
			ApplyOffset();
		}
		else if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(ASVGActor, StrokesExtrude)))
		{
			RefreshStrokeShapesExtrude();
			ApplyOffset();
		}
		else if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(ASVGActor, BevelDistance)))
		{
			RefreshFillShapesBevel();
			ApplyOffset();
		}
	}
}

void ASVGActor::ShowGenerationStartNotification()
{
	// In case and old notification is still pending
	HideGenerationStartNotification();

	FNotificationInfo Info(FText::Format(LOCTEXT("SVGBeginGeneration", "Dynamic meshes for SVG {0} are being generated..."), FText::FromString(SVGName)));
	Info.bFireAndForget = false;
	Info.bUseLargeFont = true;

	if (!DynMeshGenNotification && bIsGeneratingMeshes)
	{
		DynMeshGenNotification = FSlateNotificationManager::Get().AddNotification(Info);
		DynMeshGenNotification->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void ASVGActor::HideGenerationStartNotification()
{
	if (DynMeshGenNotification.IsValid())
	{
		DynMeshGenNotification->Fadeout();
		DynMeshGenNotification.Reset();
	}
}

void ASVGActor::ShowGenerationFinishedNotification()
{
	HideGenerationStartNotification();

	FNotificationInfo Info(FText::Format(LOCTEXT("SVGGenerated", "Dynamic meshes for SVG {0} generation finished!"), FText::FromString(SVGName)));
	Info.bFireAndForget = true;
	Info.bUseLargeFont = true;
	Info.ExpireDuration = UE::SVGImporter::Private::DefaultNotificationsExpireTime;
	Info.FadeOutDuration = UE::SVGImporter::Private::DefaultNotificationsFadeOutTime;

	FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Success);
}

void ASVGActor::ResetGeometry()
{
	if (RenderMode == ESVGRenderMode::DynamicMesh3D)
	{
		GenMeshTransaction.Reset();
		GenMeshTransaction = MakeShared<FScopedTransaction>(LOCTEXT("RegenerateFromSVG", "Regenerate Geometry from SVG Data"));
		Modify();

		bMeshesShouldBeGenerated = true;
		ScheduleShapesGeneration();
	}
}

void ASVGActor::Split()
{
	FSVGImporterUtils::SplitSVGActor(this);
}
#endif

void ASVGActor::ResetShapes()
{
	ClearShapes();
	CurrExtrudeForDepth = 0.0f;
	OffsetToCenterSVGMeshes = FVector::ZeroVector;

	UpdateFillShapesExtrude(FillsExtrude);
	UpdateStrokeShapesExtrude(StrokesExtrude);

	bSVGHasFills = false;
	bSVGHasStrokes = false;
}

void ASVGActor::ClearInstancedSVGShapes()
{
	TSet<UActorComponent*> ComponentsToBeRemoved;
	for (UActorComponent* ActorComponent : GetInstanceComponents())
	{
		if (ActorComponent && ActorComponent->IsA<USVGDynamicMeshComponent>())
		{
			ComponentsToBeRemoved.Add(ActorComponent);
		}
	}

	for (UActorComponent* Component : ComponentsToBeRemoved)
	{
		if (Component)
		{
			RemoveInstanceComponent(Component);
			Component->UnregisterComponent();
			Component->DestroyComponent();
			Component = nullptr;
		}
	}

	bClearInstanceComponents = false;
}

void ASVGActor::ClearShapes()
{
	if (bClearInstanceComponents)
	{
		ClearInstancedSVGShapes();
	}

	for (USVGDynamicMeshComponent* ShapeElement : ShapeComponents)
	{
		if (ShapeElement)
		{
			RemoveInstanceComponent(ShapeElement);
			ShapeElement->UnregisterComponent();
			ShapeElement->DestroyComponent();
			ShapeElement = nullptr;
		}
	}

	ShapeComponents.Empty();
}

void ASVGActor::CleanupEverything() const
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);

#if WITH_EDITOR
	if (GenMeshTransaction.IsValid())
	{
		GenMeshTransaction->Cancel();
	}

	if (BevelsUpdateTransaction.IsValid())
	{
		BevelsUpdateTransaction->Cancel();
	}

	if (FillsSmoothingTransaction.IsValid())
	{
		FillsSmoothingTransaction->Cancel();
	}

	if (StrokesWidthTransaction.IsValid())
	{
		StrokesWidthTransaction->Cancel();
	}
#endif
}

void ASVGActor::ResetShapesExtrudes()
{
	FillsExtrude =  UE::SVGImporter::Private::DefaultShapesExtrude;
	StrokesExtrude = UE::SVGImporter::Private::DefaultShapesExtrude;
	UpdateFillShapesExtrude(FillsExtrude);
	UpdateStrokeShapesExtrude(StrokesExtrude);
}

#undef LOCTEXT_NAMESPACE
