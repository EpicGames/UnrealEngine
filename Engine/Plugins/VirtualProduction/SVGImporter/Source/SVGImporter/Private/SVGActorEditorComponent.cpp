// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "SVGActorEditorComponent.h"
#include "SVGActor.h"

ASVGActor* USVGActorEditorComponent::GetSVGActor() const
{
	if (ASVGActor* SVGActor = GetTypedOuter<ASVGActor>())
	{
		return SVGActor;
	}

	return nullptr;
}
#endif
