// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGBakedActor.h"

// Sets default values
ASVGBakedActor::ASVGBakedActor()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;

	RootComponent = CreateDefaultSubobject<USceneComponent>("SVG_Meshes_Root");
}
