// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "SVGData.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorReimportHandler.h"
#include "SVGImporterSettings.h"
#include "SVGImporterUtils.h"
#include "SVGTypes.h"
#include "UObject/Package.h"
#endif

#if WITH_EDITOR
void USVGData::Initialize(const FSVGDataInitializer& InInitializer)
{
	SVGFileContent = InInitializer.SVGTextBuffer;
	CreateShapes(InInitializer.Elements);
	SourceFilePath = UAssetImportData::SanitizeImportFilename(InInitializer.SourceFilename, GetOutermost());
	GenerateSVGTexture();
}

void USVGData::Reset()
{
	SVGFileContent = TEXT("");
	Shapes.Empty();
}

void USVGData::GenerateSVGTexture()
{
	SVGTexture = FSVGImporterUtils::CreateSVGTexture(SVGFileContent, this);
}

void USVGData::Reimport()
{
	constexpr bool bAskForNewFileIfMissing = true;
	FReimportManager::Instance()->Reimport(this, bAskForNewFileIfMissing);

	OnSVGDataReimport().Broadcast();
}

void USVGData::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(USVGData, OverrideQuality))
	{
		bool bShouldReimport = false;
		if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			bShouldReimport = true;
		}
		else if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ToggleEditable)
		{
			ESVGSplineConversionQuality ProjectQuality = ESVGSplineConversionQuality::None;
			if (const USVGImporterSettings* SVGImporterSettings = GetDefault<USVGImporterSettings>())
			{
				ProjectQuality = SVGImporterSettings->GetSplineConversionQuality();
			}

			if (ProjectQuality != OverrideQuality)
			{
				bShouldReimport = true;
			}
		}

		if (bShouldReimport)
		{
			Reimport();
		}
	}
}

void USVGData::CreateShapes(const TArray<TSharedRef<FSVGBaseElement>>& InSVGElements)
{
	for (TSharedPtr<FSVGBaseElement> Element : InSVGElements)
	{
		if (Element->Type != ESVGElementType::Group && Element->Type != ESVGElementType::Other)
		{
			if (!Element->IsGraphicElement())
			{
				continue;
			}

			if (const TSharedPtr<FSVGGraphicsElement> SVGGraphicElem = StaticCastSharedPtr<FSVGGraphicsElement>(Element))
			{
				if (!SVGGraphicElem->IsVisible())
				{
					continue;
				}

				// Casting to path. Paths can have multiple subpaths
				if (const TSharedPtr<FSVGPath> SVGPath = StaticCastSharedPtr<FSVGPath>(Element))
				{
					FSVGShape NewShape(SVGGraphicElem->GetStyle(), SVGPath->IsClosed()
						, SVGGraphicElem->GetFillGradient()
						, SVGGraphicElem->GetStrokeGradient());

					NewShape.SetId(SVGGraphicElem->Name);

					int32 Count = 1;

					for (const FSVGSubPath& SubPath : SVGPath->SubPaths)
					{
						if (SubPath.Elements.Num() > 1)
						{
							FString CurrName = Element->Name + FString::FromInt(Count++);

							USplineComponent* SplineComponent = NewObject<USplineComponent>(GetTransientPackage(), *CurrName);

							// We need to be able to set different arrive and leave tangents per point
							SplineComponent->bAllowDiscontinuousSpline = true;

							int Key = 0;
							SplineComponent->ClearSplinePoints(true);

							for (const FSVGPathElement& PathElement : SubPath.Elements)
							{
								FSplinePoint Point = PathElement.AsSplinePoint(Key);

								// Apply  transform
								SVGGraphicElem->GetTransform().ApplyTransformToSplinePoint(Point);

								// Apply transform from parents, if any
								TSharedPtr<FSVGGroupElement> CurrParentGroup = SVGPath->GetParentGroup();
								while (CurrParentGroup)
								{
									CurrParentGroup->GetTransform().ApplyTransformToSplinePoint(Point);
									CurrParentGroup = CurrParentGroup->GetParentGroup();
								}

								SplineComponent->AddSplineLocalPoint(Point.Position);
								SplineComponent->SetTangentsAtSplinePoint(Key, Point.ArriveTangent, Point.LeaveTangent, ESplineCoordinateSpace::Local);

								Key++;
							}

							bool bIsClosed = SubPath.bIsClosed;
							NewShape.SetIsClosed(bIsClosed);

							SplineComponent->SetClosedLoop(bIsClosed);
							SplineComponent->UpdateSpline();

							TArray<FVector> Points;
							FSVGImporterUtils::ConvertSVGSplineToPolyLine(SplineComponent, ESplineCoordinateSpace::Local, GetConversionQualityFactor(), Points);

							SplineComponent->DestroyComponent();

							NewShape.AddPolygon(Points);
						}
					}

					NewShape.ApplyFillRule();
					Shapes.Add(NewShape);
				}
			}
		}
	}
}

float USVGData::GetConversionQualityFactor() const
{
	ESVGSplineConversionQuality Quality = ESVGSplineConversionQuality::None;

	if (bEnableOverrideQuality)
	{
		Quality = OverrideQuality;
	}
	else if (const USVGImporterSettings* SVGImporterSettings = GetDefault<USVGImporterSettings>())
	{
		Quality = SVGImporterSettings->GetSplineConversionQuality();
	}

	switch (Quality)
	{
		case ESVGSplineConversionQuality::VeryLow:
			return 1.e-1f;

		case ESVGSplineConversionQuality::Low:
			return 1.e-2f;

		case ESVGSplineConversionQuality::Normal:
			return 1.e-3f;

		case ESVGSplineConversionQuality::Increased:
			return 1.e-4f;

		case ESVGSplineConversionQuality::High:
			return 1.e-5f;

		case ESVGSplineConversionQuality::VeryHigh:
			return 1.e-6f;

		default:
			return 1.e-3f;
	}
}
#endif
