// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGElements.h"

#include "Components/SplineComponent.h"
#include "Math/TransformCalculus2D.h"
#include "SVGDefines.h"
#include "SVGImporterUtils.h"

using namespace UE::SVGImporter::Public;

void FSVGGraphicsElement::SetIsClosed(bool bInIsClosed)
{
	bIsClosed = bInIsClosed;
}

void FSVGGraphicsElement::Hide()
{
	Style.Hide();
}

FSVGMatrix::FSVGMatrix(const FString& InMatrixString)
{
    A  = 1.0f;
    B  = 0.0f;
    C  = 0.0f;
    D  = 1.0f;
    Tx = 0.0f;
    Ty = 0.0f;

	FSVGImporterUtils::SetSVGMatrixFromTransformString(InMatrixString, *this);

	bIsInitialized = true;
}

void FSVGMatrix::Decompose()
{
	// Based on https://frederic-wang.fr/decomposition-of-2d-transform-matrices.html

	if (bIsMatrix)
	{
		const float Det = A*D - B*C;

		TArray<FString> NewTransformList;

		// This is the same for all cases
		Translation = FVector2D(Tx, -Ty);
		NewTransformList.Add(SVGConstants::Translate);

		if (A!=0)
		{
			Skew.Y = FMath::RadiansToDegrees(FMath::Atan(B/A));
			NewTransformList.Add(SVGConstants::SkewY);

			Scale3D = FVector(1.0, A, Det/A);
			NewTransformList.Add(SVGConstants::Scale);

			Skew.X = FMath::RadiansToDegrees(FMath::Atan(C/A));
			NewTransformList.Add(SVGConstants::SkewX);
		}
		else if (B!=0)
		{
			RotAngle = 90.0;
			RotPivot = FVector2D::ZeroVector;
			NewTransformList.Add(SVGConstants::Rotate);

			Scale3D = FVector(1.0, B, Det/B);
			NewTransformList.Add(SVGConstants::Scale);

			Skew.X = FMath::RadiansToDegrees(FMath::Atan(D/B));
			NewTransformList.Add(SVGConstants::SkewX);
		}
		else // This one has 2 scalings...doesn't sound good, our system cannot handle this (this type of matrix shouldn't appear anyway)
		{
			Scale3D = FVector(1.0, C, D);
			NewTransformList.Add(SVGConstants::Scale);

			Skew.X = FMath::RadiansToDegrees(FMath::Atan(45.0));
			NewTransformList.Add(SVGConstants::SkewX);

			Scale3D = FVector(1.0, 0, 1);
			NewTransformList.Add(SVGConstants::Scale);
		}

		// Store transformations, inverted
		for (int32 i = (NewTransformList.Num() - 1); i >= 0; i--)
		{
			TransformationsList.Add(NewTransformList[i]);
		}
	}
}

FShear2D FSVGMatrix::GetShear() const
{
	return FShear2D::FromShearAngles(Skew); 
}

void FSVGMatrix::ApplyTransformToSplinePoint(FSplinePoint& OutSplinePoint)
{
	for (FString& Type : TransformationsList)
	{
		if (Type.Equals(SVGConstants::Translate))
		{
			OutSplinePoint.Position += FVector(0, Translation.X, -Translation.Y);

			// We don't need to update tangent points, since they are relative and they will move with the point itself
		}
		else if (Type.Equals(SVGConstants::Rotate))
		{
			const FVector RotPivot3D(0.0, RotPivot.X, -RotPivot.Y);

			FSVGImporterUtils::RotateAroundCustomPivot(OutSplinePoint.Position, RotPivot3D, RotAngle);

			// We rotate arrive and leave tangent points around their base, since they are relative to the point
			FSVGImporterUtils::RotateAroundCustomPivot(OutSplinePoint.ArriveTangent, FVector::ZeroVector, RotAngle);
			FSVGImporterUtils::RotateAroundCustomPivot(OutSplinePoint.LeaveTangent, FVector::ZeroVector, RotAngle);
		}
		else if (Type.Equals(SVGConstants::Scale))
		{
			OutSplinePoint.Position      *= Scale3D;
			OutSplinePoint.ArriveTangent *= Scale3D;
			OutSplinePoint.LeaveTangent  *= Scale3D;
		}
		else if (Type.Equals(SVGConstants::SkewX))
		{
			FShear2D Shear = FShear2D::FromShearAngles(Skew);
			const FVector2D ShearVec = FVector2D(Shear.GetVector());

			// SVG X -Y becomes Unreal Y Z

			OutSplinePoint.Position      += FVector(0, -OutSplinePoint.Position.Z * ShearVec.X,0.0);
			OutSplinePoint.ArriveTangent += FVector(0, -OutSplinePoint.ArriveTangent.Z * ShearVec.X, 0.0);
			OutSplinePoint.LeaveTangent  += FVector(0, -OutSplinePoint.LeaveTangent.Z * ShearVec.X, 0.0);
		}
		else if (Type.Equals(SVGConstants::SkewY))
		{
			FShear2D Shear = FShear2D::FromShearAngles(Skew);
			const FVector2D ShearVec = FVector2D(Shear.GetVector());

			OutSplinePoint.Position      += FVector(0, 0,	  -OutSplinePoint.Position.Y * ShearVec.Y);
			OutSplinePoint.ArriveTangent += FVector(0, 0, -OutSplinePoint.ArriveTangent.Y * ShearVec.Y);
			OutSplinePoint.LeaveTangent  += FVector(0, 0	, -OutSplinePoint.LeaveTangent.Y * ShearVec.Y);
		}
	}
}

void FSVGMatrix::ApplyTransformToPoint2D(FVector2D& OutPoint)
{
	if (bIsMatrix)
	{
		OutPoint.X = A*OutPoint.X + C*OutPoint.Y + Tx;
		OutPoint.Y = B*OutPoint.X + D*OutPoint.Y + Ty;
	}
	else
	{
		for (FString& Type : TransformationsList)
		{
			if (Type.Equals(SVGConstants::Translate))
			{
				OutPoint += Translation;

				// We don't need to update tangent points, since they are relative and they will move with the point itself
			}
			else if (Type.Equals(SVGConstants::Rotate))
			{
				OutPoint -= RotPivot;
				OutPoint = OutPoint.GetRotated(RotAngle);
				OutPoint+= RotPivot;
			}
			else if (Type.Equals(SVGConstants::Scale))
			{
				OutPoint*=Scale;
			}
			else if (Type.Equals(SVGConstants::SkewX))
			{
				FShear2D Shear = FShear2D::FromShearAngles(Skew);
				const FVector2D ShearVec = FVector2D(Shear.GetVector());
				OutPoint += FVector2D(OutPoint.Y*ShearVec.X, 0);
			}
			else if (Type.Equals(SVGConstants::SkewY))
			{
				FShear2D Shear = FShear2D::FromShearAngles(Skew);
				const FVector2D ShearVec = FVector2D(Shear.GetVector());
				OutPoint += FVector2D(0, OutPoint.X*ShearVec.Y);
			}
		}
	}
}

FSVGGraphicsElement::FSVGGraphicsElement()
{
	bIsGraphicElement = true;
	Transform = FSVGMatrix();

	// Type will be more precisely determined by derived types
	Type = ESVGElementType::Other;
}

void FSVGGraphicsElement::SetTransform(const FString& InMatrixSVGString)
{
	Transform = FSVGMatrix(InMatrixSVGString);
}

void FSVGGraphicsElement::TryRegisterWithParent(const TSharedPtr<FSVGBaseElement>& InParent)
{
	if (!InParent)
	{
		return;
	}

	if (InParent->Type == ESVGElementType::Group)
	{
		const TSharedRef<FSVGGroupElement> ParentAsGroup = StaticCastSharedRef<FSVGGroupElement>(InParent.ToSharedRef());
		ParentGroup = ParentAsGroup;
		ParentAsGroup->AddChild(SharedThis(this));
	}
}
