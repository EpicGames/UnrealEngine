// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGGraphicalElements.h"

#include "SVGDefines.h"
#include "SVGImporter.h"

// Used throughout the file
using namespace UE::SVGImporter::Public;

namespace UE::SVGImporter::Private
{
	static constexpr float BezierArcCircle = 0.5522847493f;
	static constexpr float OneMinusBezierArcCircle = 1.0f - BezierArcCircle;
}

//////// LINE ////////

FSVGLine::FSVGLine(float InPathLength, float InX1, float InY1, float InX2, float InY2)
	: FSVGPath(InPathLength)
	, X1(InX1), Y1(InY1), X2(InX2), Y2(InY2)
{
	Type = ESVGElementType::Line;
	ClassName = SVGConstants::Line;

	MoveTo(X1, Y1);
	LineTo(X2, Y2);
}

//////// POLYLINE ////////

FSVGPolyLine::FSVGPolyLine(float InPathLength, const TArray<FVector2D>& InPoints)
	: FSVGPath(InPathLength)
	, Points(InPoints)
{
	Type = ESVGElementType::PolyLine;
	ClassName = SVGConstants::PolyLine;

	if (Points.IsEmpty())
	{
		UE_LOGF(LogSVGImporter, Warning, "Trying to setup SVG Polygon with empty points list.");
		return;
	}

	MoveTo(Points[0].X, Points[0].Y);

	for (int32 i = 1; i < Points.Num(); i++)
	{
		LineTo(Points[i].X, Points[i].Y);
	}
}

//////// POLYGON ////////

FSVGPolygon::FSVGPolygon(float InPathLength, const TArray<FVector2D>& InPoints)
	: FSVGPolyLine(InPathLength, InPoints)
{
	Type = ESVGElementType::Polygon;
	ClassName = SVGConstants::Polygon;

	if (Points.IsEmpty())
	{
		UE_LOGF(LogSVGImporter, Warning, "Trying to setup SVG Polygon with empty points list.");
		return;
	}

	// Main polygon already created by FSVGPolyLine parent 
	// Here we only add a closing line to go back to first point, so that we have a polygon
	LineTo(Points[0].X, Points[0].Y);

	// Mark polygon as closed
	SetIsClosed(true);
}

//////// RECTANGLE ////////

FSVGRectangle::FSVGRectangle(float InX, float InY, float InWidth, float InHeight, float InRx, float InRy)
	: X(InX), Y(InY), Width(InWidth), Height(InHeight), Rx(InRx), Ry(InRy)
{
	using namespace UE::SVGImporter::Private;
	
	Type = ESVGElementType::Rectangle;
	ClassName = SVGConstants::Rect;

	// Start drawing the rectangle
	SetIsClosed(true);
	
	// use clamp instead
	if (Rx < 0.0f && Ry > 0.0f)
	{
		Rx = Ry;
	}
	
	if (Ry < 0.0f && Rx > 0.0f)
	{
		Ry = Rx;
	}
	
	if (Rx < 0.0f)
	{
		Rx = 0.0f;
	}
	
	if (Ry < 0.0f)
	{
		Ry = 0.0f;
	}
	
	if (Rx > Width/2.0f)
	{
		Rx = Width/2.0f;
	}
	
	if (Ry > Height/2.0f)
	{
		Ry = Height/2.0f;
	}

	if (Width != 0.0f && Height != 0.0f)
	{
		if (Rx < 0.00001f || Ry < 0.0001f)
		{
			MoveTo(X, Y);
			LineTo(X + Width, Y);
			LineTo(X + Width, Y + Height);
			LineTo(X, Y + Height);
			LineTo(X, Y);
		}
		else
		{
			// Rounded rectangle
			MoveTo(X + Rx, Y);
			LineTo(X + Width - Rx, Y);
			CubicBezierTo(X+Width, Y+Ry, X+Width-Rx*OneMinusBezierArcCircle, Y, X+Width, Y+Ry*OneMinusBezierArcCircle);
			LineTo(X + Width, Y+Height-Ry);
			CubicBezierTo(X+Width-Rx, Y+Height, X+Width, Y+Height-Ry*OneMinusBezierArcCircle, X+Width-Rx*OneMinusBezierArcCircle, Y+Height);
			LineTo(X+Rx, Y+Height);
			CubicBezierTo(X, Y+Height-Ry, X+Rx*OneMinusBezierArcCircle, Y+Height, X, Y+Height-Ry*OneMinusBezierArcCircle);
			LineTo(X, Y+Ry);
			CubicBezierTo(X+Rx, Y, X, Y+Ry*OneMinusBezierArcCircle, X+Rx*OneMinusBezierArcCircle, Y);
		}
	}
}

//////// CIRCLE ////////

FSVGCircle::FSVGCircle(float InCx, float InCy, float InR)
	: Cx(InCx), Cy(InCy), R(InR)
{
	using namespace UE::SVGImporter::Private;
	
	Type = ESVGElementType::Circle;	
	ClassName = SVGConstants::Circle;
	
	SetIsClosed(true);

	if (R <= 0.0f)
	{
		UE_LOGF(LogSVGImporter, Warning, "Trying to setup SVG Circle with zero or negative radius");
		return;
	}

	MoveTo(Cx+R, Cy);
	CubicBezierTo(Cx, Cy+R, Cx+R, Cy+R*BezierArcCircle, Cx+R*BezierArcCircle, Cy+R);
	CubicBezierTo(Cx-R, Cy, Cx-R*BezierArcCircle, Cy+R, Cx-R, Cy+R*BezierArcCircle);
	CubicBezierTo(Cx, Cy-R, Cx-R, Cy-R*BezierArcCircle, Cx-R*BezierArcCircle, Cy-R);
	CubicBezierTo(Cx+R, Cy, Cx+R*BezierArcCircle, Cy-R, Cx+R, Cy-R*BezierArcCircle);
}

//////// ELLIPSE ////////

FSVGEllipse::FSVGEllipse(float InCx, float InCy, float InRx, float InRy)
	: Cx(InCx), Cy(InCy), Rx(InRx), Ry(InRy)
{
	Type = ESVGElementType::Ellipse;
	ClassName = SVGConstants::Ellipse;
	
	using namespace UE::SVGImporter::Private;

	SetIsClosed(true);

	if (Rx <= 0.0f || Ry <= 0.0f)
	{
		UE_LOGF(LogSVGImporter, Warning, "Trying to setup SVG Ellipse with zero or negative value for X and/or Y radius");
		return;
	}

	if (Rx > 0.0f && Ry > 0.0f)
	{
		MoveTo(Cx+Rx, Cy);
		CubicBezierTo(Cx, Cy+Ry, Cx+Rx, Cy+Ry*BezierArcCircle, Cx+Rx*BezierArcCircle, Cy+Ry);
		CubicBezierTo(Cx-Rx, Cy, Cx-Rx*BezierArcCircle, Cy+Ry, Cx-Rx, Cy+Ry*BezierArcCircle);
		CubicBezierTo(Cx, Cy-Ry, Cx-Rx, Cy-Ry*BezierArcCircle, Cx-Rx*BezierArcCircle, Cy-Ry);
		CubicBezierTo(Cx+Rx, Cy, Cx+Rx*BezierArcCircle, Cy-Ry, Cx+Rx, Cy-Ry*BezierArcCircle);
	}
}
