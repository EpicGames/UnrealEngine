// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporter.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "SVGImporterModule"

DEFINE_LOG_CATEGORY(LogSVGImporter);

USVGData* FSVGImporterModule::CreateDefaultSVGData() const
{
	if (!OnDefaultSVGDataRequested.IsBound())
	{
		return nullptr;
	}

	return OnDefaultSVGDataRequested.Execute();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FSVGImporterModule, SVGImporter)
