// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterSettings.h"

USVGImporterSettings::USVGImporterSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("SVG Importer");
}
