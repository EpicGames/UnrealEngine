// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "SVGData.h"
#include "SVGImporterSettings.generated.h"

UCLASS(config=Engine, meta=(DisplayName="SVG Importer"))
class USVGImporterSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USVGImporterSettings();

	ESVGSplineConversionQuality GetSplineConversionQuality() const { return SplineConversionQuality; }

private:
	/**
	 * The quality used to convert SVG Spline Data into poly lines
	 */
	UPROPERTY(Config, EditAnywhere, Category="SVG")
	ESVGSplineConversionQuality SplineConversionQuality = ESVGSplineConversionQuality::VeryHigh;
};
