// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterUtils.h"
#include "Components/SplineComponent.h"
#include "Engine/World.h"
#include "GeometryScript/MeshBasicEditFunctions.h"
#include "GeometryScript/MeshMaterialFunctions.h"
#include "ProceduralMeshes/JoinedSVGDynamicMeshComponent.h"
#include "SVGData.h"
#include "SVGDefines.h"
#include "SVGEngineSubsystem.h"
#include "SVGImporter.h"
#include "SVGJoinedShapesActor.h"
#include "SVGShapeActor.h"
#include "SVGShapesParentActor.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "GeometryScript/CreateNewAssetUtilityFunctions.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformFileManager.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "ProceduralMeshes/SVGFillComponent.h"
#include "ProceduralMeshes/SVGStrokeComponent.h"
#include "SVGActor.h"
#include "SVGBakedActor.h"
#include "ScopedTransaction.h"
#include "UObject/SavePackage.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <nanosvg.h>
#include <nanosvgrast.h>
THIRD_PARTY_INCLUDES_END

#define LOCTEXT_NAMESPACE "SVGImporterUtils"

void RemoveInline(FString& InOutString, const TArray<ANSICHAR>& InCharsToRemove)
{
	const int32 StringLength = InOutString.Len();
	if (StringLength == 0)
	{
		return;
	}

	TCHAR* RawData = InOutString.GetCharArray().GetData();
	int32 CopyToIndex = 0;
	for (int32 CopyFromIndex = 0; CopyFromIndex < StringLength; ++CopyFromIndex)
	{
		if (!InCharsToRemove.Contains(RawData[CopyFromIndex]))
		{
			// Copy any character OTHER than the ones listed in CharsToRemove.
			RawData[CopyToIndex] = RawData[CopyFromIndex];
			++CopyToIndex;
		}
	}

	// Copy null-terminating character.
	if (CopyToIndex <= StringLength)
	{
		RawData[CopyToIndex] = TEXT('\0');
		InOutString.GetCharArray().SetNum(CopyToIndex + 1, EAllowShrinking::No);
	}
}

void RemoveInline(FString& InOutString, ANSICHAR CharToRemove)
{
	const TArray<ANSICHAR>& CharsToRemove = {CharToRemove};
	RemoveInline(InOutString, CharsToRemove);
}


// This is custom implementation of USplineComponent::ConvertSplineToPolyLine, which was failing for certain SVGs (see comment inside)
bool FSVGImporterUtils::ConvertSVGSplineToPolyLine(const USplineComponent* InSplineComponent, ESplineCoordinateSpace::Type InCoordinateSpace,
	float InMaxSquareDistanceFromSpline, TArray<FVector>& OutPoints)
{
	const int32 NumSegments = InSplineComponent->GetNumberOfSplineSegments();
	OutPoints.Empty();
	OutPoints.Reserve(NumSegments * 2); // We sub-divide each segment in at least 2 sub-segments, so let's start with this amount of points

	TArray<FVector> SegmentPoints;
	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		const double StartDist = InSplineComponent->GetDistanceAlongSplineAtSplinePoint(SegmentIndex);
		const double StopDist = InSplineComponent->GetDistanceAlongSplineAtSplinePoint(SegmentIndex + 1);

		// We don't want to work on extremely tiny segments
		if (StopDist - StartDist > InMaxSquareDistanceFromSpline)
		{
			if (InSplineComponent->ConvertSplineSegmentToPolyLine(SegmentIndex, InCoordinateSpace, InMaxSquareDistanceFromSpline, SegmentPoints))
			{
				if (OutPoints.Num() > 0)
				{
					// The following check was failing for some SVG shapes in the original function from Splines API, need to investigate
					// See USplineComponent::ConvertSplineToPolyLine
					// check(OutPoints.Last() == SegmentPoints[0]); // our last point must be the same as the new segment's first

					if (OutPoints.Last() == SegmentPoints[0])
					{
						OutPoints.RemoveAt(OutPoints.Num() - 1);
					}
				}
				OutPoints.Append(SegmentPoints);
			}
		}
	}

	return !OutPoints.IsEmpty();
}

bool FSVGImporterUtils::ShouldPolygonBeDrawn(const FSVGPathPolygon& ShapeToCheck, const TArray<FSVGPathPolygon>& Polygons, bool& bOutIsClockwise)
{
	// We are not doing this check pixel by pixel as a rasterizer would do,
	// but we are doing it polygon by polygon.
	// So, the approach is to manually count the direction of the polygon perimeter,
	// and start checking for segments intersection from outside of the polygon.
	// To do that, we compute the rightmost vertex, and start the check from slightly outside the shape starting from it
	// The chosen direction is 1,1

	UE::Math::TVector2<double> RightMost = ShapeToCheck.GetPolygon().GetVertices()[0];

	for (const UE::Math::TVector2<double>& Vertex : ShapeToCheck.GetPolygon().GetVertices())
	{
		if (RightMost.Y < Vertex.Y)
		{
			RightMost = Vertex;
		}
		else if (RightMost.Y == Vertex.Y)
		{
			if (RightMost.X > Vertex.X)
			{
				RightMost = Vertex;
			}
		}
	}

	static constexpr int32 PolyscanRayLength = 10000;
	static constexpr float PolyscanRayStartOffset = 0.001f;

	const UE::Math::TVector2<double> Start = RightMost + UE::Math::TVector2<double>(1, 1) * PolyscanRayStartOffset;
	const UE::Math::TVector2<double> End = Start + UE::Math::TVector2<double>(1, 1) * PolyscanRayLength;

	UE::Geometry::TSegment2<double> Ray;
	Ray.SetStartPoint(Start);
	Ray.SetEndPoint(End);

	// Since we start our ray from outside the polygon, the ray will never intersect it,
	// so we need to account ourselves for the count update
	int32 Count;
	if (ShapeToCheck.GetPolygon().IsClockwise())
	{
		Count = 1;
		bOutIsClockwise = true;
	}
	else
	{
		Count = -1;
		bOutIsClockwise = false;
	}

	// Let's check the Ray against all other shapes
	for (const FSVGPathPolygon& Shape : Polygons)
	{
		if (Shape.GetPolygon().GetVertices() == ShapeToCheck.GetPolygon().GetVertices())
		{
			continue;
		}
		
		TArray<UE::Geometry::TSegment2<double>> Intersections;
		if (FindIntersectingSegments(Ray, Shape.GetPolygon(), Intersections))
		{
			// Looping through all the segments Seg which intersect with Ray - we need them to count
			for (const UE::Geometry::TSegment2<double>& Seg : Intersections)
			{
				// Consider Ray tail the origin, we move Seg back to the origin as well, calling this new vector S				
				const FVector2D S(Seg.EndPoint().X - Seg.StartPoint().X, Seg.EndPoint().Y - Seg.StartPoint().Y);

				// We make a Ray version of type FVector2D, calling it R
				const FVector2D R(Ray.Direction);

				// R x B --> gives us the direction with which S is traversing our ray (left to right or the opposite)
				if (FVector2D::CrossProduct(R, S) > 0)
				{
					Count--;
				}
				else
				{
					Count++;
				}
			}
		}
	}

	// When count is 0, the shape should not be drawn
	if (Count == 0)
	{
		return false;
	}

	return true;
}

void FSVGImporterUtils::RotateAroundCustomPivot(FVector& OutPointToRotate, const FVector& InPivotPosition, float InAngle)
{
	FVector Point = OutPointToRotate;

	Point-=InPivotPosition;

	Point = Point.RotateAngleAxis(InAngle, FVector::BackwardVector);

	Point+=InPivotPosition;

	OutPointToRotate = Point;
}

FColor FSVGImporterUtils::GetColorFromSVGString(FString InColorString)
{
	using namespace UE::SVGImporter::Public;

	FColor OutColor = FColor::White;

	if (InColorString.StartsWith(SVGConstants::HexPrefix))
	{
		// String can be something like #000 or #808000
		OutColor = FColor::FromHex(InColorString);
	}
	else if (InColorString.StartsWith(SVGConstants::RGB))
	{
		// String to parse can be something like rgb(255, 34, 20), or rgba(255, 34, 20, 56);

		InColorString.RemoveFromStart(SVGConstants::RGB);

		// In case we are dealing with rgba, also remove the a
		if (InColorString.StartsWith(SVGConstants::A))
		{
			InColorString.RemoveFromStart(SVGConstants::A);
		}

		InColorString.RemoveFromStart(TEXT("("));
		InColorString.RemoveFromEnd(TEXT(")"));

		TArray<FString> ColorValues;
		InColorString.ParseIntoArray(ColorValues, TEXT(","));

		if (ColorValues.Num() >= 3)
		{
			if (ColorValues[0].IsNumeric() && ColorValues[1].IsNumeric() && ColorValues[2].IsNumeric())
			{
				const int32 R = FCString::Atoi(*ColorValues[0]);
				const int32 G = FCString::Atoi(*ColorValues[1]);
				const int32 B = FCString::Atoi(*ColorValues[2]);

				if (ColorValues.Num() >= 4)
				{
					if (ColorValues[3].IsNumeric())
					{
						const int32 A = FCString::Atoi(*ColorValues[3]);
						OutColor = FColor(R, G, B, A);
					}
				}
				else
				{
					OutColor = FColor(R, G, B);
				}
			}
		}
	}
	else if (InColorString.StartsWith(SVGConstants::RGBA))
	{
		InColorString.RemoveFromStart(FString(SVGConstants::RGBA) + TEXT("("));
		InColorString.RemoveFromEnd(TEXT(")"));

		TArray<FString> ColorValues;
		InColorString.ParseIntoArray(ColorValues, TEXT(","));

	}
	else if (InColorString.Equals(SVGConstants::Transparent))
	{
	}
	else if (InColorString.Equals(SVGConstants::None))
	{
		// todo: handle none here, or just skip
	}
	else
	{
		if (const TCHAR** SVGColorPtr = SVGConstants::Colors.Find(*InColorString))
		{
			OutColor = FColor::FromHex(*SVGColorPtr);
		}
	}

	return OutColor;
}

TArray<FSVGStyle> FSVGImporterUtils::StylesFromCSS(FString InData)
{
	RemoveInline(InData, {' ', '\n', '\t', '\r'});

	TArray<FSVGStyle> OutStyles;

	// Make sure the delimit string is not null or empty
	const TCHAR *Start = InData.GetCharArray().GetData();
	const int32 Length = InData.Len();

	if (Start)
	{
		int32 CurrClassNameBeginIndex = 0;
		int32 CurrClassContentBeginIndex = INDEX_NONE;

		int32 CurrAttributeNameBeginIndex = INDEX_NONE;
		int32 CurrAttributeNameEndIndex = INDEX_NONE;

		int32 CurrAttributeValueBeginIndex = INDEX_NONE;

		TMap<FString, FString> CurrStyleAttributes;
		FString CurrClassName;

		// Iterate through string.
		for (int32 i = 0; i < Length; i++)
		{
			const TCHAR* CurrentCharacter = Start + i;

			// If we found a {
			if (FCString::Strncmp(CurrentCharacter, TEXT("{"),  1) == 0)
			{
				if (CurrClassContentBeginIndex == INDEX_NONE)
				{
					CurrClassContentBeginIndex = i + 1;

					const int32 CurrClassNameEndIndex = i;
					const int32 ClassNameLength = CurrClassNameEndIndex - CurrClassNameBeginIndex;
					if (ClassNameLength >= 0)
					{
						// todo: expand on this kind of syntax?  something.class
						CurrClassName = FString::ConstructFromPtrSize(Start + CurrClassNameBeginIndex, ClassNameLength);
						if (CurrClassName.StartsWith("."))
						{
							CurrClassName.RemoveAt(0);
						}

						CurrAttributeNameBeginIndex = i + 1;
					}
				}
				else
				{
					UE_LOGF(LogSVGImporter, Warning, "Unexpected '{' found while parsing SVG CSS style string.");
				}
			}
			// If we found a :
			else if (FCString::Strncmp(CurrentCharacter, TEXT(":"),  1) == 0)
			{
				if (CurrAttributeValueBeginIndex == INDEX_NONE)
				{
					CurrAttributeValueBeginIndex = i + 1;
					CurrAttributeNameEndIndex = i;
				}
				else
				{
					UE_LOGF(LogSVGImporter, Warning, "Unexpected ':' found while parsing SVG CSS style string.");
				}
			}
			// If we found a ; or } we can finalize the attribute value
			else if (FCString::Strncmp(CurrentCharacter, TEXT(";"),  1) == 0
					|| FCString::Strncmp(CurrentCharacter,  TEXT("}"), 1) == 0)
			{
				if (CurrAttributeNameEndIndex != INDEX_NONE && CurrAttributeValueBeginIndex != INDEX_NONE)
				{
					const int32 CurrAttributeValueEndIndex = i;

					const int32 AttributeNameLength = CurrAttributeNameEndIndex - CurrAttributeNameBeginIndex;
					const int32 AttributeValueLength = CurrAttributeValueEndIndex - CurrAttributeValueBeginIndex;
					if (AttributeNameLength >= 0 || AttributeValueLength >= 0)
					{
						FString AttributeName = FString::ConstructFromPtrSize(Start + CurrAttributeNameBeginIndex, AttributeNameLength);
						AttributeName.RemoveSpacesInline();

						FString AttributeValue = FString::ConstructFromPtrSize(Start + CurrAttributeValueBeginIndex, CurrAttributeValueEndIndex - CurrAttributeValueBeginIndex);
						AttributeValue.RemoveSpacesInline();

						CurrStyleAttributes.Add(AttributeName, AttributeValue);

						CurrAttributeNameBeginIndex = i + 1;
						CurrAttributeNameEndIndex = INDEX_NONE;
						CurrAttributeValueBeginIndex = INDEX_NONE;
					}
					else
					{
						UE_LOGF(LogSVGImporter, Warning, "Error parsing attribute, skipping.");
					}
				}
				else
				{
					UE_LOGF(LogSVGImporter, Warning, "Unexpected ';' found while parsing SVG CSS style string.");
				}
			}

			// If we found a } we can finalize the current style
			if (FCString::Strncmp(CurrentCharacter,  TEXT("}"), 1) == 0)
			{
				if (CurrClassContentBeginIndex != INDEX_NONE)
				{
					// Class is finished, wrap up

					// Fill this up
					FSVGStyle NewStyle; 

					// Getting ready for next class, if any
					CurrClassNameBeginIndex = i + 1;
					CurrClassContentBeginIndex = INDEX_NONE;

					NewStyle.SetName(CurrClassName);
					NewStyle.FillFromAttributesMap(CurrStyleAttributes);
					OutStyles.Add(NewStyle);

					CurrStyleAttributes.Empty();
					CurrClassName = TEXT("");
				}
				else
				{
					UE_LOGF(LogSVGImporter, Warning, "Unexpected '}' found while parsing SVG CSS style string.");
				}
			}
		}
	}

	return OutStyles;
}

bool FSVGImporterUtils::SetSVGMatrixFromTransformString(const FString& InTransformString, FSVGMatrix& OutSVGMatrix)
{
	using namespace UE::SVGImporter::Public;

	TArray<float> ParsedMatrixValues;
	ParsedMatrixValues.AddDefaulted(6);
	nsvgParseTransform(GetData(ParsedMatrixValues), StringCast<ANSICHAR>(*InTransformString).Get());

	if (ParsedMatrixValues.Num() == 6)
	{
		OutSVGMatrix.A  =  ParsedMatrixValues[0];
		OutSVGMatrix.B  =  ParsedMatrixValues[1];
		OutSVGMatrix.C  =  ParsedMatrixValues[2];
		OutSVGMatrix.D  =  ParsedMatrixValues[3];
		OutSVGMatrix.Tx =  ParsedMatrixValues[4] * SVGScaleFactor;
		OutSVGMatrix.Ty = -ParsedMatrixValues[5] * SVGScaleFactor;

		OutSVGMatrix.bIsMatrix = true;

		OutSVGMatrix.Decompose();

		return true;
	}

	return false;
}

bool FSVGImporterUtils::FindIntersectingSegments(const UE::Geometry::TSegment2<double>& InRay, const UE::Geometry::TPolygon2<double>& InOtherPoly, TArray<UE::Geometry::TSegment2<double>>& OutArray)
{
	OutArray.Empty();

	if (!InOtherPoly.Intersects(InRay))
	{
		return false;
	}

	for (const UE::Geometry::TSegment2<double>& Seg : InOtherPoly.Segments())
	{
		// This computes test twice for intersections, but Seg.Intersects() doesn't
		// create any new objects so it should be much faster for majority of segments (should profile!)
		if (InRay.Intersects(Seg)) 
		{
			OutArray.Add(Seg);
		}
	}

	return !OutArray.IsEmpty();
}

FLinearColor FSVGImporterUtils::GetAverageColor(const TArray<FLinearColor>& InColors)
{
	if (InColors.IsEmpty())
	{
		return FLinearColor::Black;
	}

	FLinearColor OutColor = FLinearColor::Transparent;
	for (const FLinearColor& Color : InColors)
	{
		OutColor += Color;
	}

	return OutColor/InColors.Num();
}

#if WITH_EDITOR
bool FSVGImporterUtils::GetAvailableFolderPath(const FString& InAssetPath, FString& OutAssetPath)
{
	FString AssetPath = InAssetPath;

	// We don't want this to go on forever! 100 seems a big enough number
	constexpr uint32 MaxTries = 100;

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();	
	FString MyConfigDirectory = FPaths::ProjectContentDir();
	FString DirectoryToCheck = FPackageName::LongPackageNameToFilename(*AssetPath);

	if (FileManager.DirectoryExists(*DirectoryToCheck))
	{
		uint32 Count = 1;

		FString TempPath = AssetPath + FString(TEXT("_") + FString::FromInt(Count));
		DirectoryToCheck = FPackageName::LongPackageNameToFilename(*TempPath);

		while (FileManager.DirectoryExists(*DirectoryToCheck))
		{
			Count++;

			if (Count > MaxTries)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("MaxTries"),  MaxTries);
				static FText MessageTitle(NSLOCTEXT("SVGBakeActorDialog", "SVGBakeActor", "SVG actor baking dialog"));
				const EAppReturnType::Type Ret = FMessageDialog::Open( EAppMsgType::Ok,
						FText::Format(NSLOCTEXT("SVGBakeActor"
								, "BakeError"
								, "Error while baking Actor from SVG Actor! \n Folder already exists. \n Are there more than {MaxTries} Baked Blueprints of this SVG in the same folder?"), Args), MessageTitle);
				return false;
			}

			TempPath = AssetPath + FString(TEXT("_") + FString::FromInt(Count));
			DirectoryToCheck = FPackageName::LongPackageNameToFilename(*TempPath);
		}

		AssetPath = TempPath;
	}

	AssetPath.Append(TEXT("/"));
	OutAssetPath = AssetPath;

	return true;
}

UTexture2D* FSVGImporterUtils::CreateSVGTexture(const FString& InSVGString, UObject* InOuter)
{
	if (InSVGString.IsEmpty() || !InOuter)
	{
		return nullptr;
	}

	// Bytes per pixel (RGBA)
	constexpr static int32 SVG_BPP = 4;
	constexpr static int32 SVG_DPI = 96;

	NSVGimage* Image = nsvgParse(TCHAR_TO_ANSI(*InSVGString), "px", SVG_DPI);

	if (!Image)
	{
		return nullptr;
	}

	// Get max dimension, and then round to upper power of 2
	const float MaxDim = FMath::Max(Image->width, Image->height);
	const float TextureDim = FMath::RoundUpToPowerOfTwo(MaxDim);

	const float Scale = TextureDim/MaxDim;
	const float XOffset = Scale * FMath::Max(0, (MaxDim - Image->width)/2.0f);
	const float YOffset = Scale * FMath::Max(0, (MaxDim - Image->height)/2.0f);

	TArray<uint8> PixelData;
	PixelData.AddUninitialized(TextureDim * TextureDim * SVG_BPP);
	const int32 Stride = TextureDim * SVG_BPP;
	NSVGrasterizer* Rasterizer = nsvgCreateRasterizer();

	if (!Rasterizer)
	{
		return nullptr;
	}

	nsvgRasterizeFull(Rasterizer, Image, XOffset, YOffset, Scale, Scale, PixelData.GetData(), TextureDim, TextureDim, Stride);

	nsvgDeleteRasterizer(Rasterizer);
	nsvgDelete(Image);

	const FString BaseTextureName = FString(TEXT("SVGTexture"));
	const FName TextureName = MakeUniqueObjectName(InOuter, UTexture2D::StaticClass(), FName(*BaseTextureName));

	UTexture2D* SVGTexture = NewObject<UTexture2D>(InOuter, UTexture2D::StaticClass(), TextureName, RF_Public | RF_Transactional);

	if (!SVGTexture)
	{
		return nullptr;
	}

	SVGTexture->SetPlatformData(new FTexturePlatformData());
	SVGTexture->GetPlatformData()->SizeX = TextureDim;
	SVGTexture->GetPlatformData()->SizeY = TextureDim;
	SVGTexture->GetPlatformData()->SetNumSlices(1);
	SVGTexture->GetPlatformData()->PixelFormat = EPixelFormat::PF_B8G8R8A8;

	SVGTexture->Source.Init(TextureDim, TextureDim, 1, 1, ETextureSourceFormat::TSF_BGRA8, PixelData.GetData());
	SVGTexture->UpdateResource();

	return SVGTexture;
}

void FSVGImporterUtils::BakeSVGActorToBlueprint(const ASVGActor* InSVGActorToBake, const FString& InAssetPath)
{
	FString AssetsRootPath;

	// Make sure we write to a new folder
	GetAvailableFolderPath(InAssetPath, AssetsRootPath);

	TArray<FSVGBakeElement> BakedElements;
	TMap<FString, UMaterialInstance*> GeneratedMaterialInstances;

	FScopedSlowTask MainBakeTask(3, LOCTEXT("BakeSVGActor", "Baking SVG Actor to Blueprint"));
	MainBakeTask.MakeDialog(true);

	bool bIsCanceled = false;

	// Dynamic fills generation FScopedSlowTask scope
	{
		MainBakeTask.EnterProgressFrame();

		const TArray<TObjectPtr<USVGFillComponent>>& FillComponents = InSVGActorToBake->GetFillComponents();

		FScopedSlowTask GenFillCompsMeshesTask(FillComponents.Num(), LOCTEXT("GeneratingFillComponentsMeshes", "Generating fill components static meshes"));
		GenFillCompsMeshesTask.MakeDialog();

		for (USVGFillComponent* FillComponent : FillComponents)
		{
			if (MainBakeTask.ShouldCancel())
			{
				bIsCanceled = true;
				break;
			}

			GenFillCompsMeshesTask.EnterProgressFrame();

			if (UStaticMesh* BakedMesh = BakeSVGDynamicMeshToStaticMesh(FillComponent, AssetsRootPath + TEXT("Assets/"), GeneratedMaterialInstances))
			{
				FSVGBakeElement BakeElem;
				BakeElem.Mesh = BakedMesh;
				BakeElem.Transform = InSVGActorToBake->GetFillShapesRootComponent()->GetRelativeTransform() * FillComponent->GetRelativeTransform();
				BakeElem.Name = BakedMesh->GetName();
				BakedElements.Add(BakeElem);
			}
		}
	}

	// Dynamic strokes generation FScopedSlowTask scope
	{ 
		MainBakeTask.EnterProgressFrame();

		const TArray<TObjectPtr<USVGStrokeComponent>>& StrokeComponents = InSVGActorToBake->GetStrokeComponents();

		FScopedSlowTask BakeStrokeCompsTask(StrokeComponents.Num(), LOCTEXT("GeneratingStrokeComponentsMeshes", "Generating stroke components static meshes"));
		BakeStrokeCompsTask.MakeDialog();

		for (USVGStrokeComponent* StrokeComponent : StrokeComponents)
		{
			if (MainBakeTask.ShouldCancel())
			{
				bIsCanceled = true;
				break;
			}

			BakeStrokeCompsTask.EnterProgressFrame();

			if (UStaticMesh* BakedMesh = BakeSVGDynamicMeshToStaticMesh(StrokeComponent, AssetsRootPath + TEXT("Assets/"), GeneratedMaterialInstances))
			{
				FSVGBakeElement BakeElem;
				BakeElem.Mesh = BakedMesh;
				BakeElem.Transform = InSVGActorToBake->GetStrokeShapesRootComponent()->GetRelativeTransform() * StrokeComponent->GetRelativeTransform();
				BakeElem.Name = BakedMesh->GetName();
				BakedElements.Add(BakeElem);
			}
		}
	}

	// Last section of the main slow task starts here
	MainBakeTask.EnterProgressFrame();

	// This is not needed anymore
	GeneratedMaterialInstances.Empty();

	const FString BakedNameString = InSVGActorToBake->GetSVGName() + TEXT("_Baked");
	const FString TempActorPackageName = AssetsRootPath + BakedNameString;
	UPackage* Package = CreatePackage(*TempActorPackageName);

	if (IsValid(Package))
	{
		Package->FullyLoad();

		FScopedSlowTask BakeSVGActorTask(BakedElements.Num(), LOCTEXT("BakeSVGActorToBP", "Adding baked meshes to generated Blueprint"));
		BakeSVGActorTask.MakeDialog();

		const FName BakedName = FName(BakedNameString);

		ASVGBakedActor* BakedActor = NewObject<ASVGBakedActor>(Package, BakedName, RF_Public | RF_Standalone);

		if (IsValid(BakedActor) && !bIsCanceled)
		{
			for (const FSVGBakeElement& Elem : BakedElements)
			{
				if (MainBakeTask.ShouldCancel())
				{
					bIsCanceled = true;
					break;
				}

				BakeSVGActorTask.EnterProgressFrame();

				BakedActor->SVGBakeElements.Add(Elem);

				FString PrefixStr = TEXT("SVG_StaticMesh");

				// adding instance components so the blueprint will be created with these already setup and attached
				UStaticMeshComponent* NewSVGComp = NewObject<UStaticMeshComponent>(BakedActor, FName(*Elem.Name));
				NewSVGComp->SetupAttachment(BakedActor->GetRootComponent());
				BakedActor->AddInstanceComponent(NewSVGComp);
				NewSVGComp->SetStaticMesh(Elem.Mesh);
				NewSVGComp->SetRelativeTransform(Elem.Transform);
			}

			FKismetEditorUtilities::FCreateBlueprintFromActorParams Params;
			Params.ParentClassOverride = ASVGBakedActor::StaticClass();
			Params.bReplaceActor = false;
			Params.bOpenBlueprint = false;

			const FString BlueprintPath = AssetsRootPath + (TEXT("BP_") + BakedNameString);

			bool bAlreadyExists = FPackageName::DoesPackageExist(BlueprintPath);

			if (!bAlreadyExists && !bIsCanceled)
			{
				UBlueprint* NewActorBP = FKismetEditorUtilities::CreateBlueprintFromActor(BlueprintPath, BakedActor, Params);
				FAssetRegistryModule::AssetCreated(NewActorBP);

				if (IsValid(NewActorBP))
				{
					NewActorBP->GetPackage()->FullyLoad();

					if (UPackage* BP_Package = NewActorBP->GetPackage())
					{
						BP_Package->SetDirtyFlag(true);

						const FString BlueprintAssetFilename = FPackageName::LongPackageNameToFilename(BP_Package->GetPathName(), FPackageName::GetAssetPackageExtension());
						FSavePackageArgs SavePackageArgs;
						SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
						UPackage::SavePackage(BP_Package, nullptr, *BlueprintAssetFilename, SavePackageArgs);
					}
				}
			}
		}
	}

	if (MainBakeTask.ShouldCancel())
	{
		bIsCanceled = true;
	}

	if (bIsCanceled)
	{
		FFileManagerGeneric::Get().DeleteDirectory(*AssetsRootPath, true, true);
	}
}

UStaticMesh* FSVGImporterUtils::BakeSVGDynamicMeshToStaticMesh(USVGDynamicMeshComponent* InSVGDynamicMeshToBake, const FString& InAssetPath, TMap<FString, UMaterialInstance*>& OutGeneratedMaterialInstances)
{
	EGeometryScriptOutcomePins OutResult;

	FGeometryScriptCreateNewStaticMeshAssetOptions Options;

	FGeometryScriptUniqueAssetNameOptions NamingOptions;
	NamingOptions.UniqueIDDigits = 2;
	const FString FolderPath = InAssetPath;
	const FString AssetName = TEXT("SM_") + InSVGDynamicMeshToBake->GetName();

	FString BakedMeshUniqueName = AssetName;
	FString BakedMeshUniquePathAndName = FPaths::Combine(FolderPath, BakedMeshUniqueName);

	UStaticMesh* BakedMesh = UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewStaticMeshAssetFromMesh(InSVGDynamicMeshToBake->GetDynamicMesh(), BakedMeshUniquePathAndName, Options, OutResult);

	if (OutResult == EGeometryScriptOutcomePins::Failure)
	{
		static FText MessageTitle(NSLOCTEXT("SVGBakeMeshDialog", "SVGBake", "SVG mesh baking dialog"));
		const EAppReturnType::Type Ret = FMessageDialog::Open( EAppMsgType::Ok,
				FText(NSLOCTEXT("SVGBakeMesh"
						, "BakeError"
						, "Error while baking static mesh! \n Sorry about that :(") ), MessageTitle);
	}
	else
	{
		BakedMeshUniqueName.RemoveFromStart(TEXT("SM_"));

		FString MaterialPackageName = FolderPath + TEXT("MI_") + BakedMeshUniqueName;
		UPackage* MaterialInstancePackage = CreatePackage(*MaterialPackageName);

		if (IsValid(MaterialInstancePackage))
		{
			MaterialInstancePackage->FullyLoad();
			UMaterialInstance* NewMaterialInstance = nullptr;

			if (OutGeneratedMaterialInstances.Num() > 0)
			{
				bool bAlreadyExists = false;
				FString PackageNameFromKey;
				for (const TPair<FString, UMaterialInstance*>& MaterialElem : OutGeneratedMaterialInstances)
				{
					if (MaterialElem.Value->Equivalent(InSVGDynamicMeshToBake->GetMeshMaterialInstance()))
					{
						bAlreadyExists = true;
						PackageNameFromKey = MaterialElem.Key;
					}
				}
				if (bAlreadyExists)
				{
					const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

					FName MatPackageName(*PackageNameFromKey);

					TArray<FAssetData> OutAssetData;
					AssetRegistryModule.Get().GetAssetsByPackageName(MatPackageName, OutAssetData);

					if (OutAssetData.Num() > 0)
					{
						NewMaterialInstance = CastChecked<UMaterialInstance>(OutAssetData[0].GetAsset());
					}
				}
			}

			if (!IsValid(NewMaterialInstance))
			{
				FObjectDuplicationParameters MaterialInstanceDuplicateParams(InSVGDynamicMeshToBake->GetMeshMaterialInstance(), MaterialInstancePackage);
				MaterialInstanceDuplicateParams.DestName = FName(TEXT("MI_") + BakedMeshUniqueName);
				MaterialInstanceDuplicateParams.ApplyFlags = RF_Public | RF_Standalone;

				NewMaterialInstance = CastChecked<UMaterialInstance> (StaticDuplicateObjectEx(MaterialInstanceDuplicateParams));

				if (IsValid(NewMaterialInstance))
				{
					FAssetRegistryModule::AssetCreated(NewMaterialInstance);
					MaterialInstancePackage->SetDirtyFlag(true);
					const FString MaterialInstanceAssetFileName = FPackageName::LongPackageNameToFilename(MaterialInstancePackage->GetPathName(), FPackageName::GetAssetPackageExtension());
					FSavePackageArgs SavePackageArgs;
					SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
					UPackage::SavePackage(MaterialInstancePackage, NewMaterialInstance, *MaterialInstanceAssetFileName, SavePackageArgs);

					OutGeneratedMaterialInstances.Add(MaterialPackageName, InSVGDynamicMeshToBake->GetMeshMaterialInstance());
				}
			}

			if (IsValid(NewMaterialInstance))
			{
				if (UPackage* BakedMeshPackage = BakedMesh->GetPackage())
				{
					// Update material slot
					BakedMesh->SetMaterial(0, NewMaterialInstance);

					BakedMeshPackage->SetDirtyFlag(true);
					const FString BakedMeshAssetFileName = FPackageName::LongPackageNameToFilename(BakedMeshPackage->GetPathName(), FPackageName::GetAssetPackageExtension());
					FSavePackageArgs BakedMeshSavePackageArgs;
					BakedMeshSavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
					UPackage::SavePackage(BakedMeshPackage, nullptr, *BakedMeshAssetFileName, BakedMeshSavePackageArgs);	
				}
			}
		}
	}

	return BakedMesh;
}

void ShowSVGActionErrorMessage()
{
	FMessageDialog::Open(EAppMsgType::Ok
		, LOCTEXT("SVGActionErrorMessage", "Error while executing action on SVG Actor(s.")
		, LOCTEXT("SVGActionErrorMessageTitle", "SVG Actors Action Error"));
}

ASVGShapesParentActor* FSVGImporterUtils::SplitSVGActor(ASVGActor* InSVGActor)
{
	if (!InSVGActor)
	{
		ShowSVGActionErrorMessage();
		return nullptr;
	}

	UWorld* World = InSVGActor->GetWorld();
	if (!World)
	{
		ShowSVGActionErrorMessage();
		return nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("SplitSVGActor", "Split SVG Actor"));
	InSVGActor->Modify();

	const FString SplitActorName = InSVGActor->GetActorNameOrLabel() + TEXT("_Split");

	FActorSpawnParameters ParentActorSpawnParameters;
	ParentActorSpawnParameters.Name = MakeUniqueObjectName(InSVGActor->GetOuter(), ASVGShapesParentActor::StaticClass(), FName(SplitActorName));
	ASVGShapesParentActor* ShapesParentActor = World->SpawnActor<ASVGShapesParentActor>(ParentActorSpawnParameters);

	if (!ShapesParentActor)
	{
		ShowSVGActionErrorMessage();
		return nullptr;
	}

	ShapesParentActor->SetActorLabel(SplitActorName);

	USceneComponent* ShapesParentActorRootComponent = ShapesParentActor->GetRootComponent();
	if (!ShapesParentActorRootComponent)
	{
		ShapesParentActorRootComponent = NewObject<USceneComponent>(ShapesParentActor);

		ShapesParentActor->AddInstanceComponent(ShapesParentActorRootComponent);
		ShapesParentActor->SetRootComponent(ShapesParentActorRootComponent);
	}

	for (const TObjectPtr<USVGFillComponent>& FillComponent : InSVGActor->GetFillComponents())
	{
		if (FillComponent)
		{
			FSVGImporterUtils::CreateSVGShapeActorFromShape(InSVGActor, FillComponent, ShapesParentActor);
		}
	}

	for (const TObjectPtr<USVGStrokeComponent>& StrokeComponent : InSVGActor->GetStrokeComponents())
	{
		if (StrokeComponent)
		{
			FSVGImporterUtils::CreateSVGShapeActorFromShape(InSVGActor, StrokeComponent, ShapesParentActor);
		}
	}

	ShapesParentActor->SetActorTransform(InSVGActor->GetActorTransform());

	USVGEngineSubsystem::OnSVGActorSplit().ExecuteIfBound(ShapesParentActor);

#if WITH_EDITOR
	GEditor->SelectNone(true, true);
	GEditor->SelectActor(ShapesParentActor, false, true);
#endif

	InSVGActor->Destroy();

	return ShapesParentActor;
}

ASVGJoinedShapesActor* FSVGImporterUtils::ConsolidateSVGActor(ASVGActor* InSVGActor)
{
	if (!InSVGActor)
	{
		ShowSVGActionErrorMessage();
		return nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("JoinSVGActor", "Join SVG Actor"));
	InSVGActor->Modify();

	TArray<UDynamicMeshComponent*> ShapesToJoin;
	for (const TObjectPtr<UDynamicMeshComponent> ShapeComponent : InSVGActor->GetSVGDynamicMeshes())
	{
		if (ShapeComponent)
		{
			ShapesToJoin.Add(ShapeComponent);
		}
	}

	const FString JoinedActorName = InSVGActor->GetActorNameOrLabel() + TEXT("Joined");
	ASVGJoinedShapesActor* JoinedShapesActor = CreateSVGJoinedActor(InSVGActor->GetOuter(), ShapesToJoin, InSVGActor->GetActorTransform(), FName(JoinedActorName));

	if (!JoinedShapesActor)
	{
		return nullptr;
	}

	JoinedShapesActor->GetDynamicMeshComponent()->SetSVGIsUnlit(InSVGActor->bSVGIsUnlit);

#if WITH_EDITOR
	GEditor->SelectNone(true, true);
#endif

	InSVGActor->Destroy();

#if WITH_EDITOR
	GEditor->SelectActor(JoinedShapesActor, true, true);
#endif

	return JoinedShapesActor;
}

ASVGJoinedShapesActor* FSVGImporterUtils::CreateSVGJoinedActor(UObject* InOuter, const TArray<UDynamicMeshComponent*>& InShapesToJoin, const FTransform& InTransform, const FName& InBaseName)
{
	if (!InOuter)
	{
		return nullptr;
	}

	UWorld* World = InOuter->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	FActorSpawnParameters ParentActorSpawnParameters;
	ParentActorSpawnParameters.Name = MakeUniqueObjectName(InOuter, ASVGJoinedShapesActor::StaticClass(), InBaseName);

	ASVGJoinedShapesActor* JoinedShapesActor = World->SpawnActor<ASVGJoinedShapesActor>(ParentActorSpawnParameters);
	JoinedShapesActor->SetActorTransform(InTransform);

	if (UJoinedSVGDynamicMeshComponent* SVGJoinedMeshComponent = JoinedShapesActor->GetDynamicMeshComponent())
	{
		UDynamicMesh* SVGMesh = SVGJoinedMeshComponent->GetDynamicMesh();
		if (!SVGMesh)
		{
			return nullptr;
		}

		SVGMesh->Reset();
		FJoinedSVGMeshParameters MeshParams;

		int32 MaterialIDCount = 0;
		TArray<UMaterialInterface*> SVGMaterialSet;

		for (const TObjectPtr<UDynamicMeshComponent> ShapeDynamicMeshComponent : InShapesToJoin)
		{
			if (!ShapeDynamicMeshComponent)
			{
				continue;
			}

			UDynamicMesh* ShapeDynamicMesh = ShapeDynamicMeshComponent->GetDynamicMesh();
			if (!ShapeDynamicMesh)
			{
				continue;
			}

			const TArray<UMaterialInterface*>& Materials = ShapeDynamicMeshComponent->GetMaterials();
			if (Materials.Num() > 1)
			{
				TArray<UMaterialInterface*> CompactedMaterials;
				UGeometryScriptLibrary_MeshMaterialFunctions::CompactMaterialIDs(ShapeDynamicMesh, Materials, CompactedMaterials);
			}

			bool bHasMaterialID = false;
			int32 MaxMaterialID = UGeometryScriptLibrary_MeshMaterialFunctions::GetMaxMaterialID(ShapeDynamicMesh, bHasMaterialID);
			const int32 MaterialID_Offset = MaterialIDCount;

			// Store material info from shape, before remapping Material IDs
			for (int32 CurrentMaterialID = 0; CurrentMaterialID <= MaxMaterialID; CurrentMaterialID++)
			{
				if (UMaterialInterface* Material = ShapeDynamicMeshComponent->GetMaterial(CurrentMaterialID))
				{
					FSVGShapeParameters ShapeParams;
					Material->GetVectorParameterValue(FMaterialParameterInfo(TEXT("Color")), ShapeParams.Color);
					ShapeParams.ShapeName = ShapeDynamicMeshComponent->GetName();
					ShapeParams.MaterialID = MaterialIDCount;
					MeshParams.ShapesParameters.Add(ShapeParams);

					SVGMaterialSet.Add(Material);
					MaterialIDCount++;
				}
			}

			if (MaterialID_Offset > 0)
			{
				// We cannot use UGeometryScriptLibrary_MeshMaterialFunctions::RemapMaterialIDs in the for loop above, since that would overwrite IDs remapping at each loop.
				// We offset Material IDs in one go instead:
				ShapeDynamicMesh->EditMesh([&MaterialID_Offset] (FDynamicMesh3& EditMesh)
				{
					if (UE::Geometry::FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID())
					{
						for (const int32 TriangleID : EditMesh.TriangleIndicesItr())
						{
							const int32 CurID = MaterialIDs->GetValue(TriangleID);
							MaterialIDs->SetValue(TriangleID, CurID + MaterialID_Offset);
						}
					}
				}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
			}

			// Compute transform relative to new actor
			FTransform SVGShapeWorldTransform = ShapeDynamicMeshComponent->GetComponentTransform();
			FTransform RelativeTransform = SVGShapeWorldTransform.GetRelativeTransform(InTransform);

			UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMesh(SVGMesh, ShapeDynamicMesh, RelativeTransform);
		}

		MeshParams.bIsUnlit = true;
		SVGJoinedMeshComponent->Initialize(MeshParams);
		SVGJoinedMeshComponent->ConfigureMaterialSet(SVGMaterialSet);
		SVGJoinedMeshComponent->StoreCurrentMesh();

		return JoinedShapesActor;
	}

	return nullptr;
}

ASVGShapeActor* FSVGImporterUtils::CreateSVGShapeActorFromShape(const ASVGActor* InSourceSVGActor, const USVGDynamicMeshComponent* InSVGShapeComponent, AActor* InShapesParentActor)
{
	if (!InSVGShapeComponent)
	{
		return nullptr;
	}

	UWorld* World = InSVGShapeComponent->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	FTransform SVGShapeRelativeTransform = InSVGShapeComponent->GetRelativeTransform();

	if (const USceneComponent* AttachParent = InSVGShapeComponent->GetAttachParent())
	{
		SVGShapeRelativeTransform.Accumulate(AttachParent->GetRelativeTransform());
	}

	FActorSpawnParameters ShapeActorSpawnParameters;
	ShapeActorSpawnParameters.Name = MakeUniqueObjectName(InSourceSVGActor->GetOuter(), USVGFillComponent::StaticClass(), InSVGShapeComponent->GetFName());

	ASVGShapeActor* ShapeActor = World->SpawnActor<ASVGShapeActor>(ShapeActorSpawnParameters);
	if (!ShapeActor)
	{
		return nullptr;
	}

	ShapeActor->SetShape(InSVGShapeComponent);
	ShapeActor->AttachToActor(InShapesParentActor, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
	ShapeActor->SetActorRelativeTransform(SVGShapeRelativeTransform);

	return ShapeActor;
}

ASVGJoinedShapesActor* FSVGImporterUtils::JoinSVGDynamicMeshOwners(const TArray<ASVGDynamicMeshesContainerActor*>& InDynamicSVGShapesOwners)
{
	if (InDynamicSVGShapesOwners.IsEmpty())
	{
		return nullptr;
	}

	TArray<UDynamicMeshComponent*> ShapesToJoin;
	UObject* Outer = nullptr;

	FScopedTransaction Transaction(LOCTEXT("JoinSVGShapes", "Join SVG Shapes"));

	FVector AverageLocation = FVector::ZeroVector;
	for (ASVGDynamicMeshesContainerActor* MeshesOwner : InDynamicSVGShapesOwners)
	{
		if (!MeshesOwner)
		{
			continue;
		}

		MeshesOwner->Modify();

		ShapesToJoin.Append(MeshesOwner->GetSVGDynamicMeshes());
		AverageLocation += MeshesOwner->GetActorLocation();

		if (!Outer)
		{
			Outer = MeshesOwner->GetOuter();
		}
	}

	if (ShapesToJoin.IsEmpty())
	{
		return nullptr;
	}

	AverageLocation /= InDynamicSVGShapesOwners.Num();

	FTransform NewActorTransform;
	NewActorTransform.AddToTranslation(AverageLocation);

	ASVGJoinedShapesActor* JoinedShapesActor = CreateSVGJoinedActor(Outer, ShapesToJoin, NewActorTransform, TEXT("JoinedSVGShapes"));

	if (!JoinedShapesActor)
	{
		return nullptr;
	}

#if WITH_EDITOR
	GEditor->SelectNone(true, true);
#endif

	for (AActor* ShapeActor : InDynamicSVGShapesOwners)
	{
		if (ASVGShapesParentActor* ShapesParentActor = Cast<ASVGShapesParentActor>(ShapeActor))
		{
			ShapesParentActor->DestroyAttachedActorMeshes();
		}

		ShapeActor->Destroy();
	}

#if WITH_EDITOR
	GEditor->SelectActor(JoinedShapesActor, true, true);
#endif

	return JoinedShapesActor;
}
#endif

#undef LOCTEXT_NAMESPACE
