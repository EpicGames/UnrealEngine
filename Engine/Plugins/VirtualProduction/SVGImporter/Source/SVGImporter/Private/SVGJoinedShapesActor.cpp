// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGJoinedShapesActor.h"
#include "ProceduralMeshes/JoinedSVGDynamicMeshComponent.h"

ASVGJoinedShapesActor::ASVGJoinedShapesActor()
{
	JointDynamicMeshComponent = CreateDefaultSubobject<UJoinedSVGDynamicMeshComponent>("JointDynamicMeshComponent");
	SetRootComponent(JointDynamicMeshComponent);
}

TArray<UDynamicMeshComponent*> ASVGJoinedShapesActor::GetSVGDynamicMeshes()
{
	return { GetDynamicMeshComponent() };
}

