// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGDynamicMeshesContainerActor.h"
#include "GameFramework/Actor.h"
#include "SVGJoinedShapesActor.generated.h"

class ASVGActor;
class UDynamicMeshComponent;
class UJoinedSVGDynamicMeshComponent;

UCLASS(ClassGroup=(SVG), meta=(DisplayName="SVG Joint Shapes Actor", ComponentWrapperClass))
class ASVGJoinedShapesActor : public ASVGDynamicMeshesContainerActor
{
	GENERATED_BODY()

public:
	ASVGJoinedShapesActor();

	UJoinedSVGDynamicMeshComponent* GetDynamicMeshComponent() { return JointDynamicMeshComponent; }

	//~ Begin ASVGDynamicMeshesOwnerActor
	virtual TArray<UDynamicMeshComponent*> GetSVGDynamicMeshes() override;
	//~ End ASVGDynamicMeshesOwnerActor

protected:
	UPROPERTY(Category="SVG", VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<UJoinedSVGDynamicMeshComponent> JointDynamicMeshComponent;
};
