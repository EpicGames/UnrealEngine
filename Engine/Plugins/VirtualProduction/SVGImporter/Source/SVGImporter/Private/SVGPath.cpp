// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGPath.h"
#include "SVGDefines.h"
#include "SVGImporter.h"

namespace UE::SVGImporter::Private
{
	static constexpr float UnrealSplineTangentHandleScale = 3.0f;
}

FSplinePoint FSVGPathElement::AsSplinePoint(float InKey) const
{
	using namespace UE::SVGImporter::Private;
	FSplinePoint SplinePoint;

	SplinePoint.Position = FVector(Point.X, Point.Y, 0);

	SplinePoint.ArriveTangent = (SplinePoint.Position - FVector(ArriveControlPoint.X, ArriveControlPoint.Y, 0)  ) * UnrealSplineTangentHandleScale;
	SplinePoint.LeaveTangent  = (FVector(LeaveControlPoint.X, LeaveControlPoint.Y, 0) - SplinePoint.Position) * UnrealSplineTangentHandleScale;
	SplinePoint.Type = ESplinePointType::CurveCustomTangent;
	SplinePoint.InputKey = InKey;

	SplinePoint.Position =		FVector(0.0f, SplinePoint.Position.X, -SplinePoint.Position.Y);
	SplinePoint.ArriveTangent = FVector(0.0f, SplinePoint.ArriveTangent.X, -SplinePoint.ArriveTangent.Y);
	SplinePoint.LeaveTangent =	FVector(0.0f, SplinePoint.LeaveTangent.X,  -SplinePoint.LeaveTangent.Y);
	
	return SplinePoint;
}

FSVGPath::FSVGPath()
{
	Type = ESVGElementType::Path;
	PathCursor = FSVGPathElement(FVector2D::ZeroVector);

	ClassName = UE::SVGImporter::Public::SVGConstants::Path;
}

FSVGPath::FSVGPath(float InPathLength)
	: FSVGPath()
{
	PathLength = InPathLength;
}

FSVGPath::FSVGPath(const TArray<TArray<FSVGPathCommand>>& InPaths, float InPathLength)
	: FSVGPath()
{
	Type = ESVGElementType::Path;
	
	for (const TArray<FSVGPathCommand>& Commands : InPaths)
	{
		AddSubPath(Commands);
	}
	
	PathLength = InPathLength;
}

bool FSVGPath::UpdatePreviousElementLeaveTangent(const FVector2D& InLeaveTangent)
{
	if (SubPaths.Last().Elements.Num() > 1)
	{
		const int32 PrevElemIndex =SubPaths.Last().Elements.Num() - 2;
		FSVGPathElement PrevElem = SubPaths.Last().Elements[PrevElemIndex];

		PrevElem.LeaveControlPoint = InLeaveTangent;

		SubPaths.Last().Elements[PrevElemIndex] = PrevElem;

		return true;
	}

	return false;
}

void FSVGPath::AddSubPath(const TArray<FSVGPathCommand>& Commands)
{
	for (const FSVGPathCommand& PathCommand : Commands)
	{
		switch (PathCommand.InstructionType)
		{
		case ESVGPathInstructionType::None:
			break;
				
		case ESVGPathInstructionType::MoveTo:
			PathMoveTo(PathCommand.PointTo);
			break;
				
		case ESVGPathInstructionType::ClosePath:
			PathLineTo(PathCommand.PointTo);
			if (!SubPaths.IsEmpty())
			{
				SubPaths.Last().SetIsClosed(true);
			}
			break;
				
		case ESVGPathInstructionType::LineTo:
			PathLineTo(PathCommand.PointTo);
			break;
				
		case ESVGPathInstructionType::HorizontalLineTo:				
			PathLineTo(PathCommand.PointTo);
			break;
				
		case ESVGPathInstructionType::VerticalLineTo:					
			PathLineTo(PathCommand.PointTo);
			break;
				
		case ESVGPathInstructionType::CurveTo:
			PathCubicBezierTo(PathCommand.PointTo, PathCommand.ArriveControlPoint, PathCommand.LeaveControlPoint);
			break;
				
		case ESVGPathInstructionType::SmoothCurveTo:
			PathCubicBezierTo(PathCommand.PointTo, PathCommand.ArriveControlPoint, PathCommand.LeaveControlPoint);
			break;
				
		case ESVGPathInstructionType::QuadraticCurveTo:
			PathCubicBezierTo(PathCommand.PointTo, PathCommand.ArriveControlPoint, PathCommand.LeaveControlPoint);
			break;
				
		case ESVGPathInstructionType::SmoothQuadraticCurveTo:
			PathCubicBezierTo(PathCommand.PointTo, PathCommand.ArriveControlPoint, PathCommand.LeaveControlPoint);
			break;
				
		case ESVGPathInstructionType::EllipticalArc:											
			PathCubicBezierTo(PathCommand.PointTo, PathCommand.ArriveControlPoint, PathCommand.LeaveControlPoint);
			break;
					
		default: break;
		}			
	}
}

void FSVGPath::AddPoint(float InX, float InY, float InCx = 0,float InCy = 0, float OutCx = 0, float OutCy = 0)
{
	const FVector2D Point(InX, InY);
	const FVector2D InControlPoint(InCx, InCy);
	const FVector2D OutControlPoint(OutCx, OutCy);

	AddPoint(Point, InControlPoint, OutControlPoint);
}

void FSVGPath::AddPoint(const FVector2D& Point)
{
	PathCursor = FSVGPathElement(Point, Point, Point);
	SubPaths.Last().Elements.Add(PathCursor);
}

void FSVGPath::AddPoint(const FVector2D& Point, const FVector2D& ArriveControlPoint, const FVector2D& LeaveControlPoint)
{
	PathCursor = FSVGPathElement(Point, ArriveControlPoint, LeaveControlPoint);
	SubPaths.Last().Elements.Add(PathCursor);
}

void FSVGPath::MoveTo(const FVector2D& InCursorPos)
{
	SubPaths.Add(FSVGSubPath());	
	PathCursor = FSVGPathElement(InCursorPos);
	PathCursor.bIsSubpathInitialPoint = true;
	AddPoint(InCursorPos);
}

void FSVGPath::MoveTo(float InX, float InY)
{
	MoveTo(FVector2D(InX, InY));
}

void FSVGPath::PathMoveTo(const FVector2D& InPointTo)
{
	PathMoveTo(InPointTo.X, InPointTo.Y);
}

void FSVGPath::PathMoveTo(float InX, float InY)
{
	MoveTo(FVector2D(InX, InY));	
}

void FSVGPath::LineTo(const FVector2D& PointTo)
{
	const FVector2D PointFrom = PathCursor.Point;

	FVector2D D = FVector2D(PointTo - PointFrom) / 9.0; // Could this be 3.0 * UE::SVGImporter::Private::UNREAL_SPLINE_TANGENT_HANDLE_SCALE);?

	AddPoint(PointTo, PointFrom, PointTo);
	const bool bSuccess = UpdatePreviousElementLeaveTangent(PointFrom);
}

void FSVGPath::LineTo(float InX, float InY)
{	
	LineTo(FVector2D(InX, InY));	
}

void FSVGPath::PathLineTo(float InX, float InY)
{	
	LineTo(FVector2D(InX, InY));
}

void FSVGPath::PathLineTo(const FVector2D& InPointTo)
{
	PathLineTo(InPointTo.X, InPointTo.Y);
}

void FSVGPath::CubicBezierTo(float InX, float InY, float InCx, float InCy, float OutCx, float OutCy)
{
	// By default, let's have the leave control point of the end point mirroring the in control point
	const FVector2D LeaveControlPoint = 2.0 * FVector2D(InX, InY) - FVector2D(OutCx, OutCy);
	
	AddPoint(InX, InY, OutCx, OutCy, LeaveControlPoint.X, LeaveControlPoint.Y);

	// Previous leave control point is actually the arrive control point of the bezier we are currently storing
	const bool bSuccess = UpdatePreviousElementLeaveTangent(FVector2D(InCx, InCy));

	if (!bSuccess)
	{
		UE_LOGF(LogSVGImporter, Warning, "Trying to update leave tangent for non-existing previous element in path.");
	}
}

void FSVGPath::PathCubicBezierTo(float InX, float InY, float InCx, float InCy, float OutCx, float OutCy)
{			
	CubicBezierTo(InX, InY, InCx, InCy, OutCx, OutCy);	
}

void FSVGPath::PathCubicBezierTo(const FVector2D& InPointTo, const FVector2D& InArriveControlPoint, const FVector2D& InLeaveControlPoint)
{
	PathCubicBezierTo(InPointTo.X, InPointTo.Y, InArriveControlPoint.X, InArriveControlPoint.Y, InLeaveControlPoint.X, InLeaveControlPoint.Y);
}
