// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGShapeActor.h"
#include "GameFramework/Actor.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"

ASVGShapeActor::ASVGShapeActor()
{
	SVGShape = CreateDefaultSubobject<USVGDynamicMeshComponent>(TEXT("SVGShapeComponent"));
	SetRootComponent(SVGShape);
}

void ASVGShapeActor::SetShape(const USVGDynamicMeshComponent* InShape)
{
	if (!SVGShape)
	{
		return;
	}

	SVGShape->InitializeFromSVGDynamicMesh(InShape);

#if WITH_EDITOR
	SetActorLabel(InShape->GetName());
#endif
}

TArray<UDynamicMeshComponent*> ASVGShapeActor::GetSVGDynamicMeshes()
{
	if (SVGShape)
	{
		return { GetShape() };
	}

	return {};
}
