// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGDynamicMeshesContainerActor.h"
#include "GameFramework/Actor.h"
#include "SVGShapeActor.generated.h"

class UDynamicMesh;
class UDynamicMeshComponent;
class USVGDynamicMeshComponent;

/**
 * Upon splitting SVG Actors, multiple shape actors will be created.
 * Each will have their own SVG Shape.
 */
UCLASS(ClassGroup = (SVG), meta = (DisplayName = "SVG Shape Actor", ComponentWrapperClass))
class ASVGShapeActor : public ASVGDynamicMeshesContainerActor
{
	GENERATED_BODY()

public:
	ASVGShapeActor();

	void SetShape(const USVGDynamicMeshComponent* InShape);

	USVGDynamicMeshComponent* GetShape() { return SVGShape; }

	//~ Begin ASVGDynamicMeshesOwnerActor
	virtual TArray<UDynamicMeshComponent*> GetSVGDynamicMeshes() override;
	//~ End ASVGDynamicMeshesOwnerActor

protected:
	UPROPERTY(Category = "SVG", VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<USVGDynamicMeshComponent> SVGShape;
};
