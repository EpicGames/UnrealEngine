// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGShapesParentActor.h"
#include "GameFramework/Actor.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "SVGShapeActor.h"

void ASVGShapesParentActor::GetShapes(TMap<TObjectPtr<AActor>, TObjectPtr<USVGDynamicMeshComponent>>& OutMap) const
{
	ForEachAttachedActors(
		[&OutMap](AActor* AttachedActor)
		{
			if (ASVGShapeActor* ShapeActor = Cast<ASVGShapeActor>(AttachedActor))
			{
				OutMap.Add(ShapeActor, ShapeActor->GetShape());
			}

			return true;
		});
}

void ASVGShapesParentActor::DestroyAttachedActorMeshes()
{
	Super::Destroyed();

	ForEachAttachedActors(
		[](AActor* AttachedActor)
		{
			if (ASVGShapeActor* ShapeActor = Cast<ASVGShapeActor>(AttachedActor))
			{
				ShapeActor->Destroy();
			}

			return true;
		});
}

TArray<UDynamicMeshComponent*> ASVGShapesParentActor::GetSVGDynamicMeshes()
{
	TArray<UDynamicMeshComponent*> OutShapes;

	ForEachAttachedActors(
		[&OutShapes](AActor* AttachedActor)
		{
			if (ASVGShapeActor* ShapeActor = Cast<ASVGShapeActor>(AttachedActor))
			{
				OutShapes.Add(ShapeActor->GetShape());
			}

			return true;
		});

	return OutShapes;
}
