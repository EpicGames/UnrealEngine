// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGTypes.h"
#include "SVGDefines.h"
#include "SVGImporterUtils.h"

using namespace UE::SVGImporter::Public;

void FSVGStyle::FillFromAttributesMap(const TMap<FString, FString>& StyleAttributes)
{
	for (TTuple<FString, FString> Pair : StyleAttributes)
	{
		if (Pair.Key == SVGConstants::Fill)
		{
			SetFillColor(Pair.Value);
		}
		else if (Pair.Key ==  SVGConstants::Stroke)
		{
			SetStrokeColor(Pair.Value);
		}
		else if (Pair.Key ==  SVGConstants::StrokeWidth)
		{
			SetStrokeWidth(FCString::Atof(*Pair.Value) * FSVGImporterUtils::SVGScaleFactor * 0.5f);
		}
		else if (Pair.Key == SVGConstants::Display)
		{
			SetDisplay(Pair.Value);
		}
	}
}

void FSVGStyle::SetName(const FString& InName)
{
	Name = InName;
}

void FSVGStyle::SetStrokeColor(const FColor& InColor)
{
	StrokeColor = InColor;
	bHasStroke = true;
}

void FSVGStyle::SetStrokeColor(const FString& InColorSVGString)
{
	if (InColorSVGString.Equals( SVGConstants::None) || InColorSVGString.Equals( SVGConstants::Transparent))
	{
		bHasStroke = false;
		return;
	}
	
	const FColor Color = FSVGImporterUtils::GetColorFromSVGString(InColorSVGString);
	SetStrokeColor(Color);
}

void FSVGStyle::SetStrokeWidth(float InWidth)
{
	StrokeWidth = InWidth;
}

void FSVGStyle::SetFillColor(const FColor& InColor)
{
	FillColor = InColor;
	bHasFill = true;
}

void FSVGStyle::SetFillColor(const FString& InColorSVGString)
{
	if (InColorSVGString.Equals( SVGConstants::None) || InColorSVGString.Equals( SVGConstants::Transparent))
	{
		bHasFill = false;
		return;
	}
	
	const FColor Color = FSVGImporterUtils::GetColorFromSVGString(InColorSVGString);
	SetFillColor(Color);
	bHasFill = true;
}

void FSVGStyle::SetDisplay(const FString& InDisplay)
{
	Display = InDisplay;
}

void FSVGStyle::Hide()
{
	SetDisplay(SVGConstants::None);
}

bool FSVGStyle::IsVisible() const
{
	if (DisplayIsSet())
	{
		return Display != SVGConstants::None;
	}
	else
	{
		return true;
	}
}

bool FSVGStyle::DisplayIsSet() const
{
	return !Display.IsEmpty();
}

FColor FSVGGradient::GetAverageColor() const
{
	// A basic average in place of a real gradient

	if (!StopNodes.IsEmpty())
	{
		// we need to use a regular vector since FColor clamps its values to 255

		FVector4 ColorVector = FVector4(0.0f, 0.0f, 0.0f, 0.0f);
		for (const FSVGGradientStop& Stop : StopNodes)
		{
			const FColor& NodeColor = Stop.StopColor;

			ColorVector.X += NodeColor.R;
			ColorVector.Y += NodeColor.G;
			ColorVector.Z += NodeColor.B;
			ColorVector.W += NodeColor.A;
		}

		const int32 NodesCount = StopNodes.Num();

		ColorVector /= NodesCount;

		FColor OutColor;
		OutColor.R = ColorVector.X;
		OutColor.G = ColorVector.Y;
		OutColor.B = ColorVector.Z;
		OutColor.A = ColorVector.W;
		return OutColor;
	}

	return FColor::Black;
}

void FSVGPathPolygon::SetVertices(const TArray<FVector>& InVertices)
{
	Vertices2D.Empty(InVertices.Num());

	for (const FVector Vertex : InVertices)
	{
		Vertices2D.Add(FVector2D(Vertex.Z, Vertex.Y));
	}

	Polygon.AppendVertices(Vertices2D);
}

void FSVGShape::AddPolygon(const TArray<FVector>& InPoints)
{
	if (InPoints.Num() >= 3)
	{
		const FSVGPathPolygon ShapePolygon(InPoints);
		Polygons.Add(ShapePolygon);
	}
}

void FSVGShape::ApplyFillRule()
{
	// We currently apply a "polygon" version of the nonzero fill rule.
	// The way these rules are originally defined for SVGs seems to be reliant on a 2D usage of SVG data.
	// Our polygon based implementation cannot properly handle certain shape "cut" configurations.
	// In particular, this applies to overlapping geometry e.g. multiple rectangles intersecting and "cutting" each other.

	for (FSVGPathPolygon& SubShape : Polygons)
	{
		bool bShouldDrawFill = false;
		if (PolygonsNum() == 1)
		{
			bShouldDrawFill = HasFill();
		}
		else if (FSVGImporterUtils::ShouldPolygonBeDrawn(SubShape, Polygons, bIsClockwise))
		{
			bShouldDrawFill = true;
		}

		SubShape.SetShouldBeDrawn(bShouldDrawFill);
	}

	// It may happen that "cut" shapes are defined before the shape they actually cut.
	// In this case, if the "cut" shape is fully contained by the "cut-destination" shape
	// we try to swap them, so that the geometry operations later on can properly cut the geometry.
	// This might not account for 100% of cases (e.g. overlapping shapes, see comment above)

	int32 CurrShapeCount = 0;
	for (FSVGPathPolygon& SubShape : Polygons)
	{
		if (!SubShape.GetShouldBeDrawn())
		{
			int32 OtherShapeCount = 0;
			for (FSVGPathPolygon& OtherShape : Polygons)
			{
				if (CurrShapeCount != OtherShapeCount)
				{
					if (OtherShape.GetPolygon().Contains(SubShape.GetPolygon().GetVertices()))
					{
						if (OtherShapeCount > CurrShapeCount)
						{
							// Let's swap the shapes. Like this, the cut shape will be handled after the shape it needs to cut.
							// The boolean modifier will be able to actually cut the geometry.
							Polygons.Swap(OtherShapeCount, CurrShapeCount);
						}
					}
				}

				OtherShapeCount++;
			}
		}

		CurrShapeCount++;
	}
}
