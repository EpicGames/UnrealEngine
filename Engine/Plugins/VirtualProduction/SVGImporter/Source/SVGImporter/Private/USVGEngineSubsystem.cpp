// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGEngineSubsystem.h"
#include "Engine/Engine.h"

FOnSVGActorSplit USVGEngineSubsystem::OnSVGActorSplitDelegate;
FOnSVGShapesUpdated USVGEngineSubsystem::OnSVGShapesUpdatedDelegate;

USVGEngineSubsystem* USVGEngineSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<USVGEngineSubsystem>();
	}
	return nullptr;
}
