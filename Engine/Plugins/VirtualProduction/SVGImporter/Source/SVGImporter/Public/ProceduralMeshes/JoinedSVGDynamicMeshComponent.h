// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGBaseDynamicMeshComponent.h"
#include "UObject/ObjectPtr.h"
#include "JoinedSVGDynamicMeshComponent.generated.h"

class UMaterial;

USTRUCT()
struct FSVGShapeParameters
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="SVG")
	FString ShapeName;

	UPROPERTY(EditAnywhere, Category="SVG", meta=(SVGShapeParamColor))
	FLinearColor Color = FLinearColor::Black;

	UPROPERTY()
	int32 MaterialID = 0;

	bool operator==(const FSVGShapeParameters& InShapeParameters) const
	{
		return ShapeName == InShapeParameters.ShapeName
			&& Color == InShapeParameters.Color
			&& MaterialID == InShapeParameters.MaterialID;
	}

	friend uint32 GetTypeHash(const FSVGShapeParameters& InShapeParameters)
	{
		return GetTypeHash(InShapeParameters.MaterialID);
	}
};

USTRUCT()
struct FJoinedSVGMeshParameters
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsUnlit = true;

	UPROPERTY(EditAnywhere, Category="SVG")
	TSet<FSVGShapeParameters> ShapesParameters;
};

UENUM()
enum class EJoinedSVGMeshColoring : uint8 
{
	SeparateColors,
	SingleColor
};

UCLASS(MinimalAPI, ClassGroup=(Custom), meta=(BlueprintSpawnableComponent))
class UJoinedSVGDynamicMeshComponent : public USVGBaseDynamicMeshComponent
{
	friend class FJoinedSVGDynamicMeshComponentCustomization;

	GENERATED_BODY()

public:
	UJoinedSVGDynamicMeshComponent();

	void Initialize(const FJoinedSVGMeshParameters& InJoinedMeshParameters);

	void StoreCurrentMesh();

	//~ Begin UObject
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditImport() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void SetSVGIsUnlit(bool bInSVGIsUnlit);

	bool IsSVGUnlit() const { return bSVGIsUnlit; }

	void SetMainColor(const FLinearColor& InColor);

	const FLinearColor& GetMainColor() const { return MainColor; }

protected:
	void LoadStoredMesh();

	void LoadMaterialSetParameters();

	void StoreMaterialSetParameters();

	void RefreshSVGIsUnlit();

	void LoadResources();

	void UpdateMaterials(bool bInRefreshInstances = false);

	UPROPERTY()
	TObjectPtr<UDynamicMesh> SVGStoredMesh;

	UPROPERTY(EditAnywhere, Category="SVG")
	EJoinedSVGMeshColoring Coloring;

	UPROPERTY(EditAnywhere, Category="SVG", Setter="SetMainColor", Getter="GetMainColor", meta=(EditCondition="Coloring == EJoinedSVGMeshColoring::SingleColor", EditConditionHides))
	FLinearColor MainColor;

	UPROPERTY(EditAnywhere, EditFixedSize, Category="SVG", meta=(ShowOnlyInnerProperties, EditCondition="Coloring == EJoinedSVGMeshColoring::SeparateColors", EditConditionHides, AllowPrivateAccess="true"))
	TSet<FSVGShapeParameters> ShapeParametersList;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> ShapesMaterials;

	UPROPERTY(Transient)
	TObjectPtr<UMaterial> MeshMaterial_Lit;

	UPROPERTY(Transient)
	TObjectPtr<UMaterial> MeshMaterial_Unlit;

	UPROPERTY(EditAnywhere, Category="SVG", Setter="SetSVGIsUnlit", Getter="IsSVGUnlit")
	bool bSVGIsUnlit;
};
