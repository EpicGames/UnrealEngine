// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DynamicMeshComponent.h"
#include "SVGBaseDynamicMeshComponent.generated.h"

UCLASS()
class USVGBaseDynamicMeshComponent : public UDynamicMeshComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	USVGBaseDynamicMeshComponent();

	//~ Begin UActorComponent
	virtual void OnRegister() override;
	//~ End UActorComponent

protected:
	void MarkSVGMeshUpdated();

private:
	bool bMeshHasBeenUpdated;
};
