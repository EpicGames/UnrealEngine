// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGBaseDynamicMeshComponent.h"
#include "SVGDynamicMeshComponent.generated.h"

class UMaterial;
UENUM(BlueprintType)
enum class ESVGExtrudeType : uint8
{
	None,
	FrontFaceOnly,
	FrontBackMirror     UMETA(DisplayName="Front-Back Mirrored")
};

UENUM()
enum class ESVGEditMode : uint8
{
	Interactive,
	ValueSet
};

UENUM()
enum class ESVGMaterialType : uint8
{
	Default,
	Custom
};

/** @note: the below two enums should match the native ones exactly, so they can be cast directly */

UENUM()
enum class EPolygonOffsetJoinType : uint8
{
	Square, /** Uniform squaring on all convex edge joins. */
	Round,  /** Arcs on all convex edge joins. */
	Miter,  /** Squaring of convex edge joins with acute angles ("spikes"). Use in combination with MiterLimit. */
};

DECLARE_DELEGATE(FSVGMeshActionDelegate)

class UStaticMesh;

UCLASS(MinimalAPI, ClassGroup=(SVG))
class USVGDynamicMeshComponent : public USVGBaseDynamicMeshComponent
{
	GENERATED_BODY()

public:
	USVGDynamicMeshComponent();

	/** Calling this function will flatten the SVG Shape, converting it to a 2D polygon */
	SVGIMPORTER_API void FlattenShape();

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
	virtual void PostEditImport() override;
#endif
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject

	/** Calling this function will scale the SVG Shape */
	void ScaleShape(float InScale);

	/** Initialize the current SVG Dynamic Mesh Component using the data from another component of the same type */
	void InitializeFromSVGDynamicMesh(const USVGDynamicMeshComponent* InOtherSVGDynamicMeshComponent);

	void LoadResources();

	/** Sets the minimum value for extrusion. The Extrude variable will be applyed to this value, will also refresh */
	void SetMinExtrudeValue(float InBaseExtrude);
	void SetBevel(float InBevel);

	void SetExtrudeType(const ESVGExtrudeType InExtrudeType);

	void SetMeshEditMode(const ESVGEditMode InEditMode);
	void RefreshBevel();

	void StoreCurrentMesh() const;
	void LoadStoredMesh();

	void SetIsUnlit(bool bInIsUnlit);

	float GetExtrudeDepth() const { return MinExtrudeValue + Extrude; }

	virtual FName GetShapeType() const { return TEXT("Base"); }

	UMaterialInstanceDynamic* GetMeshMaterialInstance() { return MeshMaterialInstance; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG")
	FColor Color;

	UPROPERTY(VisibleAnywhere, Category="SVG", meta=(EditCondition="ExtrudeType != ESVGExtrudeType::None", EditConditionHides))
	ESVGExtrudeType ExtrudeType = ESVGExtrudeType::None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", Meta = (ClampMin = -20.0f, UIMin = -20.0f, UIMax = 20.0f, Delta = 0.01f), meta=(EditCondition="ExtrudeType != ESVGExtrudeType::None", EditConditionHides))
	float Extrude;

	UPROPERTY()
	float Bevel;

#if WITH_EDITORONLY_DATA
	bool bIsBevelBeingEdited;
#endif

protected:
	UPROPERTY()
	TObjectPtr<UMaterial> MeshMaterial;

	UPROPERTY()
	TObjectPtr<UMaterial> MeshMaterial_Unlit;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> MeshMaterialInstance;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> AssignedMaterial;

	UPROPERTY()
	FColor SVGColor;

	UPROPERTY()
	float DefaultExtrude;

	UPROPERTY()
	float MinExtrudeValue;

	UPROPERTY()
	FVector ExtrudeDirection;

	UPROPERTY(EditAnywhere, Category="Rendering")
	bool bSVGIsUnlit = false;

	UPROPERTY()
	float Scale;

#if WITH_EDITOR
	UFUNCTION(CallInEditor, Category="SVG")
	virtual void ResetToSVGValues();

	UFUNCTION(CallInEditor, Category="SVG")
	virtual UStaticMesh* BakeStaticMesh();
#endif

	void RefreshCustomMaterial();

	void RefreshColor();

	void ApplyExtrudePreview();

	void ApplyScale();

	/** Extrudes the dynamic mesh, also taking into account the base extrude value */
	virtual void SetExtrudeInternal(float InExtrude);

	UFUNCTION(BlueprintCallable, Category="SVG")
	void SetColor(FColor InColor);

	void CreateSVGMaterialInstance();

	float LastValueSetMinExtrude;

	float LastValueSetExtrude;

	UPROPERTY()
	FVector InternalCenter;

	ESVGEditMode CurrentEditMode;

	virtual void RegisterDelegates() {}
	virtual void RegenerateMesh() {}

	/** Re-applies current extrusion value */
	void RefreshExtrude();
	void SetCustomMaterial();

	// refreshes material settings (color, lit/unlit, custom material)
	void RefreshMaterial();

	void ApplySimpleUVPlanarProjection();
	void Simplify();
	void CenterMesh();

	FSVGMeshActionDelegate OnApplyMeshExtrudeDelegate;
	FSVGMeshActionDelegate OnApplyMeshBevelDelegate;
	FSVGMeshActionDelegate OnApplyMeshBevelPreviewDelegate;

	UPROPERTY()
	ESVGMaterialType MaterialType = ESVGMaterialType::Default;

	/**
	 * We serialize a copy of the mesh so that it is valid among sessions.
	 * This is needed when the geometry of a specialized SVG Shape (derived from USVGDynamicMeshComponent)
	 * is stored as a plain _new_ USVGDynamicMeshComponent, so its geometry cannot be re-generated using specialized functions from derived classes (e.g. after splitting SVG Actors)
	 */
	UPROPERTY()
	TObjectPtr<UDynamicMesh> SVGStoredMesh;
};
