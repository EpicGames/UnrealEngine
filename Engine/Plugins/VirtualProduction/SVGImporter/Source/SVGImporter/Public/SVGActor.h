// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGDynamicMeshesContainerActor.h"
#include "GameFramework/Actor.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"

#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include "SVGActor.generated.h"

#if WITH_EDITOR
class FScopedTransaction;
class FTransactionObjectEvent;
class USVGActorEditorComponent;
#endif

class USVGData;
class USVGFillComponent;
class USVGStrokeComponent;
struct FSVGPathPolygon;
struct FSVGShape;

UENUM()
enum class ESVGRenderMode : uint8 
{
	DynamicMesh3D = 0   UMETA(DisplayName = "3D"),
	Texture2D = 1       UMETA(DisplayName = "2D")
};

struct SVGIMPORTER_API FSVGActorInitGuard
{
	FSVGActorInitGuard();
	~FSVGActorInitGuard();

private:
	bool bPreviousValue;
};

UCLASS()
class SVGIMPORTER_API ASVGActor : public ASVGDynamicMeshesContainerActor
{
	GENERATED_BODY()

public:
	ASVGActor();

	void Initialize();

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostRegisterAllComponents() override;
	virtual void BeginDestroy() override;
	virtual void Destroyed() override;
	virtual void Serialize(FArchive& InArchive) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
#endif
	//~ End UObject

	//~ Begin ASVGDynamicMeshesOwnerActor
	virtual TArray<UDynamicMeshComponent*> GetSVGDynamicMeshes() override;
	//~ End ASVGDynamicMeshesOwnerActor

#if WITH_EDITOR
	USVGActorEditorComponent* GetSVGEditorComponent() { return SVGEditorComponent; }
#endif

	TArray<TObjectPtr<USVGFillComponent>> GetFillComponents() const;
	TArray<TObjectPtr<USVGStrokeComponent>> GetStrokeComponents() const;

	TObjectPtr<USceneComponent> GetFillShapesRootComponent() const { return FillShapesRoot; }
	TObjectPtr<USceneComponent> GetStrokeShapesRootComponent() const { return StrokeShapesRoot; }

	const FString& GetSVGName() const { return SVGName; }

	float GetScale() const { return Scale; }
	void SetScale(float InScale);

	float GetShapesOffset() const { return ShapesOffset; }
	void SetShapesOffset(float InShapesOffset);

	float GetFillsExtrude() const { return FillsExtrude; }
	void SetFillsExtrude(float InFillsExtrude);

	float GetStrokesExtrude() const { return StrokesExtrude; }
	void SetStrokesExtrude(float InStrokesExtrude);

	float GetBevelDistance() const { return BevelDistance; }
	void SetBevelDistance(float InBevelDistance);

	float GetStrokesWidth() const { return StrokesWidth; }
	void SetStrokesWidth(float InStrokesWidth);

	bool GetVisible() const { return bVisible; }
	void SetVisible(bool bInVisible);

protected:
	void RefreshFillShapesExtrude(const EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);
	void RefreshStrokeShapesExtrude(const EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);
	void RefreshFillShapesBevel(EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);
	void RefreshAllShapes();
	void RefreshAllShapesMaterials();
	void RefreshAllShapesCastShadow();
	void RefreshSVGPlaneCastShadow() const;

#if WITH_EDITOR
	void DisplayMissingSVGDataError(const FString& InErrorMsg);

	UFUNCTION(CallInEditor, Category="SVG", meta=(ToolTip="Creates a Blueprint with all the SVG components as Static Meshes"
		, EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	void BakeToBlueprint() const;

	UFUNCTION(CallInEditor, Category="SVG", meta=(ToolTip="Regenerates geometry from the referenced SVG Data Asset"
		, EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	void ResetGeometry();

	UFUNCTION(CallInEditor, Category="SVG", meta=(ToolTip="Splits this SVGActor into multiple Actors, each with a single SVG Shape component."
		, EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	void Split();
#endif

	void Generate();
	void TryLoadDefaultSVGData();
	void LoadResources();
	void DestroySVGDynMeshComponents();
	void RefreshPlaneMaterialInstance();
	void DestroySVGPlane();
	void CreateSVGPlane();
	void ResetShapesExtrudes();
	void CleanupEverything() const;
	bool CanUpdateShapes() const;
	bool CanGenerateShapes() const;
	void RemoveAllDelegates() const;

	void OnShapesGenerationEnd();
	void OnShapesExtrudeEnd();

	void CreateMeshesFromShape(const FSVGShape& InShape);

	void AddStrokeComponent(const TArray<FVector>& InPoints, float InThickness, const FColor& InColor, bool bIsClosed, bool bIsClockwise
		, float InExtrudeOffset = 0.0f, const FString& InName = TEXT(""));

	void AddFillComponent(const TArray<FSVGPathPolygon>& InShapesToDraw, const FColor& InColor, float InExtrudeOffset = 0.0f, const FString& InName = TEXT(""));

	void UpdateFillShapesSmoothing();
	void UpdateFillShapesSmoothingEnable();
	void UpdateStrokesVisibility();
	void UpdateStrokesWidth();

	void ResetShapes();
	void ClearShapes();
	void ClearInstancedSVGShapes();

	void UpdateStrokeShapesExtrude(float InExtrude, ESVGEditMode InEditMode = ESVGEditMode::ValueSet);
	void UpdateFillShapesExtrude(float InExtrude, ESVGEditMode InEditMode = ESVGEditMode::ValueSet);
	void UpdateFillShapesBevel(float InBevel, ESVGEditMode InEditMode);

	void ScheduleShapesGeneration();
	void ScheduleFillExtrudeUpdate();
	void ScheduleStrokesExtrudeUpdate();
	void ScheduleFillsSmoothUpdate(bool bInHasSmoothingFlagChanged = false);
	void ScheduleStrokesWidthUpdate();
	void ScheduleBevelsUpdate();

	void TriggerActorDetailsRefresh();
	void GenerateShapesOnEndFrame();
	void UpdateBevelsOnEndFrame();
	void UpdateFillsExtrudeOnEndFrame();
	void UpdateStrokesExtrudeOnEndFrame();

	void UpdateFillsSmoothingOnEndFrame();
	void UpdateStrokesWidthOnEndFrame();

	void GenerateNextMesh();
	void CenterMeshes();
	void ComputeOffsetScale();
	void ApplyOffset();
	void ApplyScaleAndCenter();

	void OnVisibilityChanged();
	void ShowShapes() const;
	void HideShapes() const;

#if WITH_EDITOR
	void ShowGenerationStartNotification();
	void HideGenerationStartNotification();
	void ShowGenerationFinishedNotification();
#endif

	DECLARE_MULTICAST_DELEGATE(FSVGActorActionEndDelegate);
	FSVGActorActionEndDelegate OnShapesGenerationEnd_Delegate;
	FSVGActorActionEndDelegate OnFillsExtrudeEnd_Delegate;
	FSVGActorActionEndDelegate OnStrokesExtrudeEnd_Delegate;

	// todo: we might be able to merge these 2 delegates with the more specific extrude ones above
	FSVGActorActionEndDelegate OnFillsUpdateEnd_Delegate;
	FSVGActorActionEndDelegate OnStrokesUpdateEnd_Delegate;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG")
	TObjectPtr<USVGData> SVGData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG")
	ESVGRenderMode RenderMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG"
		, Setter="SetScale", Getter= "GetScale"
		, meta=(ToolTip="Scale each SVG shape", Delta=0.01f, AllowPrivateAccess="true"))
	float Scale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG"
		, meta=(EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	ESVGExtrudeType ExtrudeType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ClampMin=0.1f, UIMin=0.1f, UIMax=20.0f, Delta=0.01f)
		, Setter="SetFillsExtrude", Getter= "GetFillsExtrude"
		, meta=(EditCondition="ExtrudeType != ESVGExtrudeType::None && bSVGHasFills==true && RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides, AllowPrivateAccess="true"))
    	float FillsExtrude;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ClampMin=0.01f, UIMin=0.01f, UIMax=20.0f, Delta=0.01f)
		, Setter="SetStrokesExtrude", Getter= "GetStrokesExtrude"
		, meta=(EditCondition="ExtrudeType != ESVGExtrudeType::None && bSVGHasStrokes==true && RenderMode == ESVGRenderMode::DynamicMesh3D && bIgnoreStrokes == false", EditConditionHides, AllowPrivateAccess="true"))
	float StrokesExtrude;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG"
		, Setter="SetShapesOffset", Getter= "GetShapesOffset"
		, meta=(ToolTip="Applies an offset to all SVG shapes along the X axis. Offset increases with each shapes, based on their order within the SVG source.", Delta=0.01f, AllowPrivateAccess="true"))
	float ShapesOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ClampMin=0.0f, UIMin=0.0f, UIMax=0.5f, Delta=0.005f)
		, Setter="SetBevelDistance", Getter= "GetBevelDistance"
		, meta=(EditCondition="ExtrudeType != ESVGExtrudeType::None && bSVGHasFills==true && RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
    float BevelDistance;

	/** This value is computed right after generating the SVG shapes, and used to scale the shapes offset based on the actor bounds extent */
	UPROPERTY()
	float ShapesOffsetScale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ToolTip="Applies an offset to all SVG fill shapes edges, resulting in smoother geometry."
		, EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	bool bSmoothFillShapes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG"
		, meta=(EditCondition="bSmoothFillShapes==true && RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides), Category="SVG")
	float SmoothingOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ToolTip="SVG strokes geometry will not be generated, even when available in SVG Data asset."
		, EditCondition="RenderMode == ESVGRenderMode::DynamicMesh3D", EditConditionHides))
	bool bIgnoreStrokes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG", meta=(ClampMin=0.01f, UIMin=0.01f, UIMax=20.0f)
		, Setter="SetStrokesWidth", Getter= "GetStrokesWidth"
		, meta=(EditCondition="bSVGHasStrokes == true && RenderMode == ESVGRenderMode::DynamicMesh3D  && bIgnoreStrokes == false", EditConditionHides))
	float StrokesWidth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SVG"
	, meta=(EditCondition="bSVGHasStrokes==true && RenderMode == ESVGRenderMode::DynamicMesh3D && bIgnoreStrokes == false", EditConditionHides))
	EPolygonOffsetJoinType StrokeJoinStyle;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(ToolTip="SVG Actor is unaffected by lighting. This option does not affect SVG shapes with a custom material."))
	bool bSVGIsUnlit;

	UPROPERTY(EditAnywhere, Category="Rendering")
	bool bSVGCastsShadow;

protected:
	UPROPERTY(EditAnywhere, Setter="SetVisible", Getter="GetVisible", Category="Rendering")
	bool bVisible = true;

	UPROPERTY(EditDefaultsOnly, Category="SVG")
	bool bSVGHasStrokes;

	UPROPERTY(EditDefaultsOnly, Category="SVG")
	bool bSVGHasFills;

	UPROPERTY()
	TObjectPtr<USceneComponent> FillShapesRoot;

	UPROPERTY()
	TObjectPtr<USceneComponent> StrokeShapesRoot;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<USVGActorEditorComponent> SVGEditorComponent;
#endif

	UPROPERTY()
	TArray<TObjectPtr<USVGDynamicMeshComponent>> ShapeComponents;

	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> SVGPlane;

	UPROPERTY()
	TObjectPtr<UStaticMesh> PlaneStaticMesh;

	UPROPERTY()
	TObjectPtr<UMaterial> PlaneMaterial;

	UPROPERTY()
	TObjectPtr<UMaterial> PlaneMaterial_Unlit;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PlaneMaterialInstance;

	UPROPERTY()
	bool bMeshesShouldBeGenerated;

	UPROPERTY()
	FString SVGName;

	bool bIsGeneratingMeshes;
	bool bIsUpdatingFills;
	bool bStrokesExtrudeFinished;
	bool bFillsExtrudeFinished;

	/** Used to perform rudimentary depth ordering */
	float CurrExtrudeForDepth = 0.0f;

	int32 GeneratedShapesCount = 0;
	int32 UpdatedFillsCount = 0;
	int32 UpdatedStrokesCount = 0;
	int32 UpdatedBevelsCount = 0;

	int32 UnnamedStrokesNum = 0;
	int32 UnnamedFillsNum = 0;

	FDelegateHandle GenMeshDelegateHandle;
	FDelegateHandle FillsUpdateDelegateHandle;
	FDelegateHandle StrokesUpdateDelegateHandle;
	FDelegateHandle BevelUpdateDelegateHandle;

UPROPERTY()
	FVector OffsetToCenterSVGMeshes;

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> GenMeshTransaction;
	TSharedPtr<FScopedTransaction> BevelsUpdateTransaction;
	TSharedPtr<FScopedTransaction> FillsSmoothingTransaction;
	TSharedPtr<FScopedTransaction> StrokesWidthTransaction;

	TSharedPtr<SNotificationItem> DynMeshGenNotification;
#endif

private:
#if WITH_EDITOR
	friend class FSVGDynamicMeshVisualizer;
	friend class FSVGActorEditorComponentVisualizer;

	void SetFillsExtrudeInteractive(float InFillsExtrude);
	void SetStrokesExtrudeInteractive(float InStrokesExtrude);
	bool bFillsExtrudeInteractiveUpdate;
	bool bStrokesExtrudeInteractiveUpdate;
#endif

	bool bClearInstanceComponents;
	bool bInitialized;
};
