// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "SVGActorEditorComponent.generated.h"

class ASVGActor;

UCLASS(MinimalAPI)
class USVGActorEditorComponent : public USceneComponent
{
	GENERATED_BODY()
#if WITH_EDITOR
public:
	SVGIMPORTER_API ASVGActor* GetSVGActor() const;
#endif
};
