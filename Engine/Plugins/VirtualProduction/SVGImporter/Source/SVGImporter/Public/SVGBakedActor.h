// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "SVGBakedActor.generated.h"

USTRUCT()
struct FSVGBakeElement
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "SVG Bake Element")
	TObjectPtr<UStaticMesh> Mesh = nullptr;

	UPROPERTY(VisibleAnywhere, Category = "SVG Bake Element")
	FTransform Transform;

	UPROPERTY(VisibleAnywhere, Category = "SVG Bake Element")
	FString Name;
};

UCLASS(MinimalAPI)
class ASVGBakedActor : public AActor
{
	GENERATED_BODY()

public:
	ASVGBakedActor();

	UPROPERTY(EditAnywhere, Category = "Baked SVG")
	TArray<FSVGBakeElement> SVGBakeElements;
};
