// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGGraphicalElements.h"
#include "UObject/Object.h"
#include "SVGData.generated.h"

class UTexture2D;

/**
 * Helper struct to provide SVGData objects with the necessary information for initialization.
 */
struct FSVGDataInitializer
{
	FSVGDataInitializer()
	{
	}

	FSVGDataInitializer(const FString& InSVGTextBuffer, const FString& InSourceFilename = TEXT(""))
		: SVGTextBuffer(InSVGTextBuffer)
		, SourceFilename(InSourceFilename)
	{
	}

	/** Is this element initialized? */
	bool IsInitialized() const
	{
		return !SVGTextBuffer.IsEmpty();
	}

	/** The SVG Text Buffer */
	FString SVGTextBuffer = TEXT("");

	/** The SVG Filename. Can be empty if Text Buffer is not coming from a file */
	FString SourceFilename = TEXT("");

	/** The SVG Elements already parsed from the SVG Text Buffer */
	TArray<TSharedRef<FSVGBaseElement>> Elements;
};

/**
 * Can be used to set the desired fidelity when converting SVG Splines into Polylines
 */
UENUM(BlueprintType)
enum class ESVGSplineConversionQuality : uint8
{
	None UMETA(Hidden),
	VeryLow,
	Low,
	Normal,
	Increased,
	High,
	VeryHigh,
};

UCLASS()
class SVGIMPORTER_API USVGData : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnSVGDataReimport);
	FOnSVGDataReimport& OnSVGDataReimport() { return OnSVGDataReimportDelegate; }

	/** Initialize this SVGData with the information provided by the initializer */
	void Initialize(const FSVGDataInitializer& InInitializer);

	void Reset();

	/** Generate a texture from the SVG information */
	void GenerateSVGTexture();
	void Reimport();

	/** Get the SVG source file path, if available */
	const FString& GetSourceFilePath() const { return SourceFilePath; }

	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;

	/** Create the shapes composing this SVG Data, starting from the information provided by SVG parsing */
	void CreateShapes(const TArray<TSharedRef<FSVGBaseElement>>& InSVGElements);
#endif

	UPROPERTY(VisibleAnywhere, Category = "SVG Data")
	TObjectPtr<UTexture2D> SVGTexture;

	UPROPERTY(VisibleAnywhere, Category = "SVG Data")
	FString SVGFileContent;

	UPROPERTY()
	TArray<FSVGShape> Shapes;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = "Source Asset")
	FString SourceFilePath = TEXT("");

	/** Enable quality overriding option */
	UPROPERTY(EditAnywhere, Category="SVG Data", meta=(InlineEditConditionToggle))
	bool bEnableOverrideQuality = false;

	/**
	 * The quality used to convert SVG Spline Data into poly lines
	 * Changing this value will trigger a re-import (existing geometry will need to be re-generated).
	 */
	UPROPERTY(EditAnywhere, Category="SVG Data", meta = (EditCondition = "bEnableOverrideQuality"))
	ESVGSplineConversionQuality OverrideQuality = ESVGSplineConversionQuality::VeryHigh;

private:
	float GetConversionQualityFactor() const;

	FOnSVGDataReimport OnSVGDataReimportDelegate;
#endif
};
