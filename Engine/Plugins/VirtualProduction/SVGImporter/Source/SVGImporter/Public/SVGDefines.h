// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"

namespace UE::SVGImporter::Public::SVGConstants
{
	//@todo: actual colors supported list is very long, see https://www.december.com/html/spec/colorsvghex.html
	static TMap<const FName, const TCHAR*> Colors =
	{
		{TEXT("black"),   TEXT("#000000")},
		{TEXT("silver"),  TEXT("#C0C0C0")},
		{TEXT("gray"),    TEXT("#808080")},
		{TEXT("grey"),    TEXT("#808080")},
		{TEXT("white"),   TEXT("#FFFFFF")},
		{TEXT("maroon"),  TEXT("#800000")},
		{TEXT("red"),     TEXT("#FF0000")},
		{TEXT("orange"),  TEXT("#FFA500")},
		{TEXT("purple"),  TEXT("#800080")},
		{TEXT("fuchsia"), TEXT("#FF00FF")},
		{TEXT("green"),   TEXT("#008000")},
		{TEXT("lime"),    TEXT("#00FF00")},
		{TEXT("olive"),   TEXT("#808000")},
		{TEXT("yellow"),  TEXT("#FFFF00")},
		{TEXT("navy"),    TEXT("#000080")},
		{TEXT("blue"),    TEXT("#0000FF")},
		{TEXT("teal"),    TEXT("#008080")},
		{TEXT("aqua"),    TEXT("#00FFFF")}
	};

	static const TCHAR* Auto              = TEXT("auto");
	static const TCHAR* Transform         = TEXT("transform");
	static const TCHAR* Translate         = TEXT("translate");
	static const TCHAR* Rotate            = TEXT("rotate");
	static const TCHAR* SkewX             = TEXT("skewX");
	static const TCHAR* SkewY             = TEXT("skewY");
	static const TCHAR* Scale             = TEXT("scale");
	static const TCHAR* Matrix            = TEXT("matrix");
	static const TCHAR* Id                = TEXT("id");
	static const TCHAR* Stroke            = TEXT("stroke");
	static const TCHAR* StrokeWidth       = TEXT("stroke-width");
	static const TCHAR* Fill              = TEXT("fill");
	static const TCHAR* Class             = TEXT("class");
	static const TCHAR* Style             = TEXT("style");
	static const TCHAR* G                 = TEXT("g"); // Group element
	static const TCHAR* SVG               = TEXT("svg");
	static const TCHAR* LinearGradient    = TEXT("linearGradient");
	static const TCHAR* RadialGradient    = TEXT("radialGradient");
	static const TCHAR* GradientStop      = TEXT("stop");
	static const TCHAR* Offset            = TEXT("offset");
	static const TCHAR* StopColor         = TEXT("stop-color");
	static const TCHAR* StopOpacity       = TEXT("stop-opacity");
	static const TCHAR* GradientUnits     = TEXT("gradientUnits");
	static const TCHAR* ClipPath          = TEXT("clipPath");          // A clip path restricts the drawing region
	static const TCHAR* UserSpaceOnUse    = TEXT("userSpaceOnUse");    // Coordinate used by a Clip Path are defined with the Clip Path itself
	static const TCHAR* ObjectBoundingBox = TEXT("objectBoundingBox"); // Coordinate used by a Clip Path are relative to the element
	static const TCHAR* Rect              = TEXT("rect");
	static const TCHAR* Circle            = TEXT("circle");
	static const TCHAR* Ellipse           = TEXT("ellipse");
	static const TCHAR* Line              = TEXT("line");
	static const TCHAR* Polygon           = TEXT("polygon");
	static const TCHAR* PolyLine          = TEXT("polyline");
	static const TCHAR* Path              = TEXT("path");
	static const TCHAR* PathLength        = TEXT("pathLength");
	static const TCHAR* D                 = TEXT("d"); // Path to be Drawn
	static const TCHAR* Points            = TEXT("points");
	static const TCHAR* X                 = TEXT("x");
	static const TCHAR* X1                = TEXT("x1"); // X value for elements requiring more than one Point to be drawn
	static const TCHAR* X2                = TEXT("x2"); // X value for elements requiring more than one Point to be drawn
	static const TCHAR* Y                 = TEXT("y");
	static const TCHAR* Y1                = TEXT("y1"); // Y value for elements requiring more than one Point to be drawn
	static const TCHAR* Y2                = TEXT("y2"); // Y value for elements requiring more than one Point to be drawn
	static const TCHAR* CX                = TEXT("cx"); // Center Point X
	static const TCHAR* CY                = TEXT("cy"); // Center Point Y
	static const TCHAR* R                 = TEXT("r");  // Radius of a circle
	static const TCHAR* RX                = TEXT("rx"); // Radius X (e.g. Ellipse)
	static const TCHAR* RY                = TEXT("ry"); // Radius Y (e.g. Ellipse)
	static const TCHAR* Width             = TEXT("width");
	static const TCHAR* Height            = TEXT("height");
	static const TCHAR* HexPrefix         = TEXT("#");
	static const TCHAR* Percentage        = TEXT("%");
	static const TCHAR* RGB               = TEXT("rgb");
	static const TCHAR* RGBA              = TEXT("rgba");
	static const TCHAR* A                 = TEXT("a"); // Alpha
	static const TCHAR* Transparent       = TEXT("transparent");
	static const TCHAR* Display           = TEXT("display");
	static const TCHAR* None              = TEXT("none");
	static const TCHAR* Inherit           = TEXT("inherit");
	static const TCHAR* URL_Start         = TEXT("url(#");
	static const TCHAR* URL_End           = TEXT(")");
}
