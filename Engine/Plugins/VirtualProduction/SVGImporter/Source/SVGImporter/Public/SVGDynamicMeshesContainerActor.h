// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "SVGDynamicMeshesContainerActor.generated.h"

class UDynamicMeshComponent;

UCLASS(MinimalAPI, Abstract)
class ASVGDynamicMeshesContainerActor : public AActor
{
	GENERATED_BODY()

public:
	virtual TArray<UDynamicMeshComponent*> GetSVGDynamicMeshes() PURE_VIRTUAL(ASVGDynamicMeshesContainerActor::GetSVGDynamicMeshes, return {};)
};
