// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGTypes.h"
#include "Math/TransformCalculus2D.h"

/**
 * The types defined in this header file are used when parsing SVG Elements
 */

struct FSplinePoint;
struct FSVGGroupElement;

enum class ESVGElementType : uint8
{
	None,
	Other,
	SVG,
	Style,
	Group,
	ClipPath,
	Path,
	Rectangle,
	Circle,
	Ellipse,
	Line,
	PolyLine,
	Polygon,
	Gradient
};

struct SVGIMPORTER_API FSVGMatrix
{
    FSVGMatrix() = default;

    explicit FSVGMatrix(const FString& InMatrixString);

	void Decompose();

	void ApplyTransformToSplinePoint(FSplinePoint& OutSplinePoint);

	void ApplyTransformToPoint2D(FVector2D& OutPoint);

	const FTransform& GetTransform() const { return FinalTransform; }

	FShear2D GetShear() const;

	float A = 0.0f;
	float B = 0.0f;
	float C = 0.0f;
	float D = 0.0f;
	float Tx = 0.0f;
	float Ty = 0.0f;

	bool bIsMatrix = false;

	FVector2D Translation = FVector2D::ZeroVector;

	FVector2D Scale = FVector2D::UnitVector;

	float RotAngle = 0.0;

	FVector2D RotPivot = FVector2D::UnitVector;

	FVector2D Skew = FVector2D::ZeroVector;

	FTransform FinalTransform = FTransform::Identity;

	TArray<FString> TransformationsList;

	FVector Scale3D = FVector::OneVector;

	bool bIsInitialized = false;
};

struct FSVGBaseElement : TSharedFromThis<FSVGBaseElement>
{
	void SetName(const FString& InName)
	{
		Name = InName;
	}

	bool TypeIsSet() const
	{
		return Type != ESVGElementType::None;
	}

	bool IsGraphicElement() const
	{
		return bIsGraphicElement;
	}

	ESVGElementType Type = ESVGElementType::None;

	bool bIsGraphicElement = false;

	FString Name;
};

struct FSVGMainElement : FSVGBaseElement
{
	FSVGMainElement()
	{
		Type = ESVGElementType::SVG;
	}
};

struct FSVGStyleElement : FSVGBaseElement
{
	FSVGStyleElement(const TArray<FSVGStyle>& InStyles)
		: Styles(InStyles)
	{
		Type = ESVGElementType::Style;
	}

	TArray<FSVGStyle> Styles;

	bool GetStyleByClassName(const FString& InClassName, FSVGStyle& OutStyle)
	{
		for (const FSVGStyle& Style : Styles)
		{
			if (Style.GetName().Equals(InClassName))
			{
				OutStyle = Style;
				return true;
			}
		}

		return false;
	}

	bool GetDefaultStyle(FSVGStyle& OutStyle)
	{
		if(!Styles.IsEmpty())
		{
			OutStyle = Styles[0];
			return true;
		}

		return false;
	}
};

struct FSVGClipPath : FSVGBaseElement
{
	FSVGClipPath()
	{
		Type = ESVGElementType::ClipPath;
	}
};

struct FSVGGradientElement : FSVGBaseElement
{
	FSVGGradientElement(const FSVGGradient& InGradient)
		: Gradient(InGradient)
	{
		Type = ESVGElementType::Gradient;
	}

	bool HasID() const
	{
		return Gradient.IsValid();
	}

	const FString& GetID() const
	{
		return Gradient.Id;
	}

	FSVGGradient Gradient;
};

/**
 * Base for all elements which can be drawn.
 * They can have fill and/or stroke information, a style, and a transform.
 */
struct SVGIMPORTER_API FSVGGraphicsElement : FSVGBaseElement
{
	FSVGGraphicsElement();

	/** If the specified parent is a Group element, add this Graphic Element to its children */
	void TryRegisterWithParent(const TSharedPtr<FSVGBaseElement>& InParent);

	/** Set the class name (e.g. rect, circle, path) */
	void SetClassName(const FString& InClassName)
	{
		ClassName = InClassName;
	}

	/** Set the style class name (the one specified by CSS styling info) */
	void SetStyleClassName(const FString& InStyleClassName)
	{
		StyleClassName = InStyleClassName;
	}

	/** Set style information */
	void SetStyle(const FSVGStyle& InStyle)
	{
		Style = InStyle;
	}

	/** Set stroke color (used if this graphic element has strokes) */
	void SetStrokeColor(const FColor& InColor)
	{
		Style.SetStrokeColor(InColor);
	}

	/** Set stroke color (used if this graphic element has strokes) */
	void SetStrokeColor(const FString& InColorSVGString)
	{
		Style.SetStrokeColor(InColorSVGString);
	}

	/** Set stroke width (used if this graphic element has strokes) */
	void SetStrokeWidth(float InWidth)
	{
		Style.SetStrokeWidth(InWidth);
	}

	/** Set fill color (used if this graphic element has strokes) */
	void SetFillColor(const FColor& InColor)
	{
		Style.SetFillColor(InColor);
	}

	/** Set fill color (used if this graphic element has strokes) */
	void SetFillColor(const FString& InColorSVGString)
	{
		Style.SetFillColor(InColorSVGString);
	}

	void SetFillGradient(const FSVGGradient& InFillGradient)
	{
		FillGradient = InFillGradient;
	}

	void SetFillGradientID(const FString& InFillGradientID)
	{
		FillGradientID = InFillGradientID;
	}

	void SetStrokeGradient(const FSVGGradient& InStrokeGradient)
	{
		StrokeGradient = InStrokeGradient;
	}

	void SetStrokeGradientID(const FString& InStrokeGradientID)
	{
		StrokeGradientID = InStrokeGradientID;
	}

	/** Initialize the transform matrix for this Graphic Element */
	void SetTransform(const FString& InMatrixSVGString);

	/** Mark the path for this element as closed or not */
	void SetIsClosed(bool bInIsClosed);

	/** Set visibility for this element and its hierarchy */
	void Hide();

	/** Whether this element and its hierarchy should be rendered or not */
	bool IsVisible() const { return Style.IsVisible(); }

	/** Is this graphic element path closed? */
	bool IsClosed() const { return bIsClosed; }

	/** Should the path of this graphic element be rendered as a stroke? */
	bool HasStroke() const { return Style.HasStroke(); }

	/** Should the path of this graphic element be rendered as a fill? */
	bool HasFill() const { return Style.HasFill(); }

	bool HasFillGradientID() const { return !FillGradientID.IsEmpty(); }

	bool HasStrokeGradientID() const { return !StrokeGradientID.IsEmpty(); }

	const FString& GetFillGradientID() const { return FillGradientID; }

	const FString& GetStrokeGradientID() const { return StrokeGradientID; }

	const FSVGGradient& GetFillGradient() const { return FillGradient; }

	const FSVGGradient& GetStrokeGradient() const { return StrokeGradient; }

	/** Get the class name for this Graphic Element (e.g. rect, circle, path) */
	const FString& GetClassName() const { return ClassName; }

	/** Get the style class name for this Graphic Element (the one specified by CSS styling info) */
	const FString& GetStyleClassName() const { return StyleClassName; }

	/** Get the fill color of this element */
	const FColor& GetFillColor() const { return Style.GetFillColor(); }

	/** Get the stroke color of this element */
	const FColor& GetStrokeColor() const { return Style.GetStrokeColor(); }

	/** Get the style of this element */
	const FSVGStyle& GetStyle() const { return Style; }

	/** Get the transform to be applied to this element */
	FSVGMatrix& GetTransform() { return Transform; }

	/** Returns the parent group element, if any. Nullptr if none. */
	const TSharedPtr<FSVGGroupElement>& GetParentGroup() const { return ParentGroup; }

protected:
	TSharedPtr<FSVGGroupElement> ParentGroup = nullptr;

	/** The class name of this graphic element (e.g. rect, circle, path) */
	FString ClassName;

	/** The Style Class name of this element (for CSS styling) */
	FString StyleClassName;

	/** The style of this element */
	FSVGStyle Style;

	/** Fill gradient, if available */
	FSVGGradient FillGradient;

	/** Stroke gradient, if available */
	FSVGGradient StrokeGradient;

	/** The transform of this element */
	FSVGMatrix Transform;

	/** Is this element path closed? */
	bool bIsClosed = false;

	FString FillGradientID;

	FString StrokeGradientID;
};

/**
 * Container struct representing SVG "group".
 * It holds a list of graphics elements.
 * It is defined as a graphic element itself, since it can contain info about elements styling.
 */
struct FSVGGroupElement : FSVGGraphicsElement
{
	TArray<TSharedRef<FSVGGraphicsElement>> Children;

	FSVGGroupElement()
	{
		Type = ESVGElementType::Group;
	}

	void Setup(const TArray<TSharedRef<FSVGGraphicsElement>>& InChildren)
	{
		Children = InChildren;
	}

	void AddChild(const TSharedRef<FSVGGraphicsElement>& InChild)
	{
		Children.AddUnique(InChild);
	}
};
