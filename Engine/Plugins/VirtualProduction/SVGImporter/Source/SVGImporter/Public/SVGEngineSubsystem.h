// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "SVGEngineSubsystem.generated.h"

class ASVGActor;
class ASVGShapesParentActor;

DECLARE_DELEGATE_OneParam(FSVGActorComponentsReady, ASVGActor*)
DECLARE_DELEGATE_OneParam(FOnSVGActorSplit, ASVGShapesParentActor*)
DECLARE_DELEGATE_OneParam(FOnSVGShapesUpdated, AActor*)

UCLASS()
class USVGEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	static USVGEngineSubsystem* Get();

	FSVGActorComponentsReady& GetSVGActorComponentsReadyDelegate() { return SVGActorComponentsReady; }

	static FOnSVGActorSplit& OnSVGActorSplit() { return OnSVGActorSplitDelegate; }
	static FOnSVGShapesUpdated& OnSVGShapesUpdated() { return OnSVGShapesUpdatedDelegate; }

protected:
	SVGIMPORTER_API static FOnSVGActorSplit OnSVGActorSplitDelegate;
	SVGIMPORTER_API static FOnSVGShapesUpdated OnSVGShapesUpdatedDelegate;

	FSVGActorComponentsReady SVGActorComponentsReady;
};
