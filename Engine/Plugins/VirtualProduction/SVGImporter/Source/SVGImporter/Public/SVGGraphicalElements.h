// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGPath.h"

/*
 * A set of graphical elements (lines, rectangles, circles, etc)
 * implemented as SVG paths (all derive from FSVGPath)
 */

struct FSVGLine : FSVGPath
{
	SVGIMPORTER_API FSVGLine(float InPathLength, float InX1, float InY1, float InX2, float InY2);

	float X1 = 0.0f;
	float Y1 = 0.0f;
	float X2 = 0.0f;
	float Y2 = 0.0f;
};

struct FSVGPolyLine : FSVGPath
{
	SVGIMPORTER_API FSVGPolyLine(float InPathLength, const TArray<FVector2D>& InPoints);

	TArray<FVector2D> Points;
};

struct FSVGPolygon : FSVGPolyLine
{
	SVGIMPORTER_API FSVGPolygon(float InPathLength, const TArray<FVector2D>& InPoints);
};

struct FSVGRectangle : FSVGPath
{
	SVGIMPORTER_API FSVGRectangle(float InX, float InY, float InWidth, float InHeight, float InRx, float InRy);

	float X;
	float Y;
	float Width;
	float Height;
	float Rx;
	float Ry;
};

struct FSVGCircle : FSVGPath
{	
	SVGIMPORTER_API FSVGCircle(float InCx, float InCy, float InR);

	float Cx;
	float Cy;
	float R;
};

struct FSVGEllipse : FSVGPath
{
	SVGIMPORTER_API FSVGEllipse(float InCx, float InCy, float InRx, float InRy);

	float Cx;
	float Cy;
	float Rx;
	float Ry;
};
