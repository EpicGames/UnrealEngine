// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectPtr.h"

class USVGData;

DECLARE_LOG_CATEGORY_EXTERN(LogSVGImporter, Log, All);

class FSVGImporterModule : public IModuleInterface
{
	friend class FSVGImporterEditorModule;

public:

	static FSVGImporterModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FSVGImporterModule>(TEXT("SVGImporter"));
	}

	USVGData* CreateDefaultSVGData() const;

protected:
	DECLARE_DELEGATE_RetVal(USVGData*, FOnDefaultSVGDataRequested)
	FOnDefaultSVGDataRequested OnDefaultSVGDataRequested;
};
