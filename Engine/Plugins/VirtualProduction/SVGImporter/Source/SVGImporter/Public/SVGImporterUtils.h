// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Polygon2.h"
#include "SegmentTypes.h"
#include "UObject/Object.h"

class AActor;
class ASVGActor;
class ASVGDynamicMeshesContainerActor;
class ASVGJoinedShapesActor;
class ASVGShapeActor;
class ASVGShapesParentActor;
class UDynamicMeshComponent;
class UMaterialInstance;
class USVGDynamicMeshComponent;
class USVGRawElement;
class USplineComponent;
class UStaticMesh;
class UTexture2D;
struct FSVGMatrix;
struct FSVGPathPolygon;
struct FSVGShape;
struct FSVGStyle;
struct FSplinePoint;

namespace ESplineCoordinateSpace
{
	enum Type : int;
}

class FSVGImporterUtils
{
public:
	inline static double SVGScaleFactor = 0.1;

	/**
	 * Given a threshold, returns a list of vertices along the specified spline that, treated as a list of segments (polyline), matches the spline shape.
	 * This function is a reviewed implementation of USplineComponent::ConvertSplineToPolyLine.
	 * @param InSplineComponent The spline to be fitted
	 * @param InCoordinateSpace The coordinate space to be used
	 * @param InMaxSquareDistanceFromSpline Fitting threshold
	 * @param OutPoints The polyline, as a list of Vectors
	 * @return 
	 */
	static bool ConvertSVGSplineToPolyLine(const USplineComponent* InSplineComponent, ESplineCoordinateSpace::Type InCoordinateSpace
		, float InMaxSquareDistanceFromSpline, TArray<FVector>& OutPoints);

	/**
	 * Evaluates if the specified ShapeToCheck is to be drawn or not (e.g. actual shape or hole to be cut)
	 * The value nonzero determines the "insideness" of a point in the shape by drawing a ray from that point to infinity in any direction,
 	 * and then examining the places where a segment of the shape crosses the ray.
 	 * Starting with a count of zero, add one each time a path segment crosses the ray from left to right
 	 * and subtract one each time a path segment crosses the ray from right to left.
 	 * After counting the crossings, if the result is zero then the point is outside the path. Otherwise, it is inside.
 	 *
 	 * See: https://developer.mozilla.org/en-US/docs/Web/Attribute/fill-rule
 	 *
 	 * For now, this function is only considering the non-zero fill rule.
 	 * todo: might need support for even odd fill rule.
 	 */
	static bool ShouldPolygonBeDrawn(const FSVGPathPolygon& ShapeToCheck, const TArray<FSVGPathPolygon>& Polygons, bool& bOutIsClockwise);

	/** Rotates an input point around a custom pivot */
	static void RotateAroundCustomPivot(FVector& OutPointToRotate, const FVector& InPivotPosition, float InAngle);

	/** Extracts a color from SVG color string */
	SVGIMPORTER_API static FColor GetColorFromSVGString(FString InColorString);

	/** Extracts SVG Style from a CSS style string */
	SVGIMPORTER_API static TArray<FSVGStyle> StylesFromCSS(FString InString);

	/** Sets an SVG Matrix based on the specified SVG Transform string */
	static bool SetSVGMatrixFromTransformString(const FString& InTransformString, FSVGMatrix& OutSVGMatrix);

	/**
	 * Computes a simple average of the provided Linear Colors
	 * 
	 * @param InColors the colors to average
	 * @return the average color
	 */
	static FLinearColor GetAverageColor(const TArray<FLinearColor>& InColors);

private:
	/**
	 * Finds the polygon segments intersecting the specified ray
	 * @param InRay The ray
	 * @param InOtherPoly The polygon to check against
	 * @param OutArray The list of intersecting segments
	 * @return true if at least one segment is found
	 */
	static bool FindIntersectingSegments(const UE::Geometry::TSegment2<double>& InRay, const UE::Geometry::TPolygon2<double>& InOtherPoly
	                                     , TArray<UE::Geometry::TSegment2<double>>& OutArray);

#if WITH_EDITOR
public:
	/** Creates a Texture 2D from the specified SVG Text Buffer. Uses the nanosvg parser and rasterizer */
	static UTexture2D* CreateSVGTexture(const FString& InSVGString, UObject* InOuter);

	/**
	 * Create Blueprint from SVG Actor.
	 * Will also create the required Static Meshes, Materials and Material Instances assets.
	 * The Dynamic meshes of the SVG Actor will be converted to Static Meshes, and their materials to Materials.
	 * Material Instances will be used when possible, to reduce the number of created Materials.
	 * @param InSVGActorToBake the Actor to be used as a template for the Blueprint
	 * @param InAssetPath The path where the Blueprint and required assets will be created
	 */
	static void BakeSVGActorToBlueprint(const ASVGActor* InSVGActorToBake, const FString& InAssetPath);

	/**
	 * Creates a Static Mesh Asset starting from the specified SVG Dynamic Mesh Component
	 * @param InSVGDynamicMeshToBake the source SVG Dynamic Mesh
	 * @param InAssetPath The path where the Asset will be created
	 * @param OutGeneratedMaterialInstances if possible, the function will generate material instances instead of multiple materials
	 */
	static UStaticMesh* BakeSVGDynamicMeshToStaticMesh(USVGDynamicMeshComponent* InSVGDynamicMeshToBake, const FString& InAssetPath, TMap<FString, UMaterialInstance*>& OutGeneratedMaterialInstances);

	/**
	 * Splits the specified SVG Actor into multiple actors. Each actor will have a single SVG Shape component
	 * 
	 * @param InSVGActor the source SVG Actor to split
	 * @return the newly created Shapes Parent Actor
	 */
	SVGIMPORTER_API static ASVGShapesParentActor* SplitSVGActor(ASVGActor* InSVGActor);

	/**
	 * Merges all SVG Shapes of this Actor into a single Actor with a single Shape.
	 * 
	 * @param InSVGActor the source SVG Actor to consolidate
	 * @return the newly created Joined Shapes Actor
	 */
	SVGIMPORTER_API static ASVGJoinedShapesActor* ConsolidateSVGActor(ASVGActor* InSVGActor);

	/**
	 * Joins the SVG Shapes of 2 or more Actors of type ASVGDynamicMeshesOwnerActor onto a single ASVGJoinedShapesActor
	 * 
	 * @param InDynamicSVGShapesOwners the list of Actors owning the SVG Shapes which are going to be joined
	 * @return the newly created Joined Shapes Actor
	 */
	SVGIMPORTER_API static ASVGJoinedShapesActor* JoinSVGDynamicMeshOwners(const TArray<ASVGDynamicMeshesContainerActor*>& InDynamicSVGShapesOwners);

private:
	/** Based on the specified Asset Path, will return an available Folder Path where we can create assets for SVG Baking purposes */
	static bool GetAvailableFolderPath(const FString& InAssetPath, FString& OutAssetPath);

	/** Creates a new SVG Joined Actor from the provided Dynamic Meshes Shapes. */
	static ASVGJoinedShapesActor* CreateSVGJoinedActor(UObject* InOuter, const TArray<UDynamicMeshComponent*>& InShapesToJoin, const FTransform& InTransform, const FName& InBaseName);

	/**
	 * Creates an SVG Shape Actor for the specified component
	 * 
	 * @param InSourceSVGActor the source SVG Actor
	 * @param InSVGShapeComponent the Shape Component to be moved to the newly created actor
	 * @param InShapesParentActor the created actor will be attached to this parent actor
	 * @return the newly created actor
	 */
	static ASVGShapeActor* CreateSVGShapeActorFromShape(const ASVGActor* InSourceSVGActor, const USVGDynamicMeshComponent* InSVGShapeComponent, AActor* InShapesParentActor);
#endif
};
