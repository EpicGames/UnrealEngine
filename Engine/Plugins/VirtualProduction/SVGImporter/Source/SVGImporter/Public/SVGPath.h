// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGElements.h"
#include "Components/SplineComponent.h"
#include "Math/TransformCalculus2D.h"

/**
 * The available commands describing a path
 */
enum class ESVGPathInstructionType : uint8
{
	None,
	MoveTo,
	ClosePath,
	LineTo,
	HorizontalLineTo,
	VerticalLineTo,
	CurveTo,
	SmoothCurveTo,
	QuadraticCurveTo,
	SmoothQuadraticCurveTo,
	EllipticalArc,
};

/**
 * Defines a single SVG Path Command
 */
struct FSVGPathCommand
{
    FSVGPathCommand() = default;

    FSVGPathCommand(ESVGPathInstructionType InInstructionType)
        : InstructionType(InInstructionType)
	{
	}

	/** The instruction type of this command (e.g. Move To, Line To, Cubic Bezier To, etc) */
	ESVGPathInstructionType InstructionType = ESVGPathInstructionType::None;

	/** Command destination point */
	FVector2D PointTo = FVector2D::UnitVector;

	/** Arrive control point */
	FVector2D ArriveControlPoint = FVector2D::ZeroVector;

	/** Leave Control point */
	FVector2D LeaveControlPoint = FVector2D::UnitVector;
};

/**
 * The Element of an SVG Path
 */
struct SVGIMPORTER_API FSVGPathElement
{
public:
	/**
	 * Returns this Path Element as a Spline Point
	 * @param InputKey the Key/Index the returned spline will be associated with
	 */
	FSplinePoint AsSplinePoint(float InputKey) const;

	/** Is this element a subpath initial point? */
	bool IsSubpathInitialPoint() const {return bIsSubpathInitialPoint;}

    FSVGPathElement() = default;

    FSVGPathElement(const FVector2D& Point, const FVector2D& InArriveControlPoint = FVector2D::ZeroVector
    	, const FVector2D& InLeaveControlPoint = FVector2D::ZeroVector)
		: Point(Point)
        , ArriveControlPoint(InArriveControlPoint)
        , LeaveControlPoint(InLeaveControlPoint)
    {
	}

	/** Point location */
	FVector2D Point = FVector2D::ZeroVector;

	/** Arrive control point */
	FVector2D ArriveControlPoint = FVector2D::ZeroVector;

	/** Leave control point */
	FVector2D LeaveControlPoint = FVector2D::ZeroVector;

	/** Is this Element the first point of a sub path? */
	bool bIsSubpathInitialPoint = false;
};

/**
 * SVG Subpath
 */
struct FSVGSubPath
{
	/** Set the path closed flag*/
	void SetIsClosed(bool bInIsClosed)
	{
		bIsClosed = bInIsClosed;
	}

	/** Add an element to this path */
	void AddPathElement(const FSVGPathElement& InElement)
	{
		Elements.Add(InElement);
	}

	/** Elements of this Sub Path */
	TArray<FSVGPathElement> Elements;

	/** True if path is closed */
	bool bIsClosed = false;
};

/**
 * SVG Path. It can describe a path with multiple subpaths and a length
 */
struct SVGIMPORTER_API FSVGPath : public FSVGGraphicsElement
{
public:
	FSVGPath();

	FSVGPath(float InPathLength);

	FSVGPath(const TArray<TArray<FSVGPathCommand>>& InPaths, float InPathLength);

	/** Set the leave tangent of the previous point */
	bool UpdatePreviousElementLeaveTangent(const FVector2D& InLeaveTangent);

	/** Add a subpath to this path */
	void AddSubPath(const TArray<FSVGPathCommand>& Commands);

	/**
	 * Add a point to this path
	 * @param InX Point X
	 * @param InY Point Y
	 * @param InCx In Control Point X
	 * @param InCy In Control Point Y
	 * @param OutCx Out Control Point X
	 * @param OutCy Out Control Point Y
	 */
	void AddPoint(float InX, float InY, float InCx, float InCy, float OutCx, float OutCy);

	/**
	 * Add a point to this path
	 * @param Point the location of the point
	 */
	void AddPoint(const FVector2D& Point);

	/**
	 * Add a point to this path
	 * @param Point the location of the point
	 * @param ArriveControlPoint the location of the Arrive Control Point
	 * @param LeaveControlPoint the location of the Leave Control Point
	 */
	void AddPoint(const FVector2D& Point, const FVector2D& ArriveControlPoint, const FVector2D& LeaveControlPoint);

	/** Move To operation (see svg specifications) */
	void MoveTo(const FVector2D& InCursorPos);
	
	/** Move To operation (see svg specifications) */
	void MoveTo(float InX, float InY);
	
	/** Path Move To operation (see svg specifications) */
	void PathMoveTo(const FVector2D& InPointTo);
	
	/** Path Move To operation (see svg specifications) */
	void PathMoveTo(float InX, float InY);
	
	/** Line To operation (see svg specifications) */
	void LineTo(const FVector2D& PointTo);
	
	/** Line To operation (see svg specifications) */
	void LineTo(float InX, float InY);
	
	/** Path Line To operation (see svg specifications) */
	void PathLineTo(float InX, float InY);
	
	/** Path Line To operation (see svg specifications) */
	void PathLineTo(const FVector2D& InPointTo);

	/** Cubic Bezier To operation (see svg specifications) */
	void CubicBezierTo(float InX, float InY, float InCx, float InCy, float OutCx, float OutCy);
	
	/** Path Cubic Bezier To operation (see svg specifications) */
	void PathCubicBezierTo(float InX, float InY, float InCx, float InCy, float OutCx, float OutCy);
	
	/** Path Cubic Bezier To operation (see svg specifications) */
	void PathCubicBezierTo(const FVector2D& InPointTo, const FVector2D& InArriveControlPoint, const FVector2D& InLeaveControlPoint);

	/** Length of this path */
	float PathLength = 0.0f;

	/** The SubPaths composing this Path*/
	TArray<FSVGSubPath> SubPaths;

	/** Current position of the Path Cursor */
	FSVGPathElement PathCursor;

	/** Starting Path point */
	FSVGPathElement InitialPoint;
};
