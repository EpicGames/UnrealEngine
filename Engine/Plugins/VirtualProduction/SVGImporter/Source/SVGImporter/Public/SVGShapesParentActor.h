// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGDynamicMeshesContainerActor.h"
#include "GameFramework/Actor.h"
#include "SVGShapesParentActor.generated.h"

class USVGDynamicMeshComponent;

/**
 * This actor is used as a parent to organize SVG Shape Actors
 */
UCLASS(MinimalAPI)
class ASVGShapesParentActor : public ASVGDynamicMeshesContainerActor
{
	GENERATED_BODY()

public:
	/** Returns a map associating each dynamic mesh component shape to the actor they are attached to */
	SVGIMPORTER_API void GetShapes(TMap<TObjectPtr<AActor>, TObjectPtr<USVGDynamicMeshComponent>>& OutMap) const;

	void DestroyAttachedActorMeshes();

	//~ Begin ASVGDynamicMeshesOwnerActor
	virtual TArray<UDynamicMeshComponent*> GetSVGDynamicMeshes() override;
	//~ End ASVGDynamicMeshesOwnerActor
};
