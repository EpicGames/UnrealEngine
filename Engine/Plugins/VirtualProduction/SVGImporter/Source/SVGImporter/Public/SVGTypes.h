// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Polygon2.h"
#include "SVGTypes.generated.h"

/**
 * The types defined in this header file are the ones used to serialize SVGData assets:
 * - FSVGStyle (describes the look of SVG shapes, think Material)
 * - FSVGPathPolygon (a polygon, part of a path)
 * - FSVGShape, explicitly used by USVGData (a shape can include multiple polygons)
 */

class USplineComponent;

USTRUCT()
struct SVGIMPORTER_API FSVGStyle
{
	GENERATED_BODY()

	void FillFromAttributesMap(const TMap<FString, FString>& StyleAttributes);

	void SetName(const FString& InName);

	void SetStrokeColor(const FColor& InColor);

	void SetStrokeColor(const FString& InColorSVGString);

	void SetStrokeWidth(float InWidth);

	void SetFillColor(const FColor& InColor);

	void SetFillColor(const FString& InColorSVGString);

	void SetDisplay(const FString& InDisplay);

	void Hide();

	bool HasStroke() const { return bHasStroke; }

	bool HasFill() const { return bHasFill || !bHasStroke; }

	const FColor& GetFillColor() const { return FillColor; }

	const FColor& GetStrokeColor() const { return StrokeColor; }

	bool IsVisible() const;

	bool DisplayIsSet() const;

	const FString& GetName() const { return Name; }

	float GetStrokeWidth() const { return StrokeWidth; }

private:
	UPROPERTY()
	FString Name;

	UPROPERTY()
	bool bHasFill = false;

	UPROPERTY()
	FColor FillColor = FColor::Black;

	UPROPERTY()
	bool bHasStroke = false;

	UPROPERTY()
	FColor StrokeColor = FColor::Black;

	UPROPERTY()
	float StrokeWidth = 1.0f;

	UPROPERTY()
	FString Display;
};

enum class ESVGGradientPointUnit : uint8
{
	UserSpaceOnUse,
	ObjectBoundingBox
};

USTRUCT()
struct FSVGGradientStop
{
	GENERATED_BODY()

	UPROPERTY()
	float Offset = 0.0f;

	UPROPERTY()
	float StopOpacity = 0.0f;

	UPROPERTY()
	FColor StopColor = FColor::Black;
};

USTRUCT()
struct FSVGGradient
{
	GENERATED_BODY()

	FSVGGradient()
	{}

	FSVGGradient(const FString& Id, ESVGGradientPointUnit InGradientUnits, const FVector2D& StartingPoint
		, const FVector2D& EndingPoint, const TArray<FSVGGradientStop>& StopNodes = {})
		: Id(Id)
		, GradientUnits(InGradientUnits)
		, StartingPoint(StartingPoint)
		, EndingPoint(EndingPoint)
		, StopNodes(StopNodes)
	{
	}

	bool IsValid() const { return !Id.IsEmpty(); }

	UPROPERTY()
	FString Id;

	// Tells whether this gradient is using coordinates or percentage values for start and end point of the gradient
	ESVGGradientPointUnit GradientUnits = ESVGGradientPointUnit::ObjectBoundingBox;

	// This can be either coordinates of percentage values, see GradientUnits value for that
	FVector2D StartingPoint = FVector2D::ZeroVector;

	// This can be either coordinates of percentage values, see GradientUnits value for that
	FVector2D EndingPoint = FVector2D::UnitVector * 100.0f;

	UPROPERTY()
	TArray<FSVGGradientStop> StopNodes;

	//
	//todo: add
	//- spreadMethod
	//- gradientTransform
	//

	SVGIMPORTER_API FColor GetAverageColor() const;
};

USTRUCT(Blueprintable)
struct FSVGPathPolygon
{
	GENERATED_BODY()

	FSVGPathPolygon()
	{
		bShouldBeDrawn = false;
	}

	FSVGPathPolygon(const TArray<FVector>& InVertices)
	{
		Vertices = InVertices;
		SetVertices(Vertices);
	}

	const TArray<FVector>& GetVertices() const { return Vertices; }

	const TArray<FVector2D>& Get2DVertices() const { return Vertices2D; }

	int GetNumVertices() const { return Polygon.VertexCount(); }

	bool GetShouldBeDrawn() const { return bShouldBeDrawn; }

	const UE::Geometry::TPolygon2<double>& GetPolygon() const { return Polygon; }

	bool ShouldBeDrawn() const { return bShouldBeDrawn; }

	void SetVertices(const TArray<FVector>& InVertices);

	void SetShouldBeDrawn(bool bInShouldBeDrawn)
	{
		bShouldBeDrawn = bInShouldBeDrawn;
	}

private:
	UE::Geometry::TPolygon2<double> Polygon;

	UPROPERTY()
	TArray<FVector2D> Vertices2D;

	UPROPERTY()
	TArray<FVector> Vertices;

	UPROPERTY()
	bool bShouldBeDrawn;
};

USTRUCT()
struct FSVGShape
{
	GENERATED_BODY()

	FSVGShape()
	{
		SetDefaultValues();
	}

	FSVGShape(const FSVGStyle& InStyle, bool bInIsClosed
		, const FSVGGradient& InFillGradient = FSVGGradient()
		, const FSVGGradient& InStrokeGradient = FSVGGradient())
	{
		SetDefaultValues();
		Style = InStyle;
		bIsClosed = bInIsClosed;
		FillGradient = InFillGradient;
		StrokeGradient = InStrokeGradient;
	}

	bool IsVisible() const	{ return Style.IsVisible(); }

	void SetDefaultValues()
	{
		bIsClosed = false;
		bShapesHaveBeenGenerated = false;
	}

	void SetId(const FString& InId)
	{
		Id = InId;
	}

	void SetIsClosed(bool bInIsClosed)
	{
		bIsClosed = bInIsClosed;
	}

	FColor GetStrokeColor() const
	{
		if (StrokeGradient.IsValid())
		{
			return StrokeGradient.GetAverageColor();
		}
		else
		{
			return Style.GetStrokeColor();
		}
	}

	FColor GetFillColor() const
	{
		if (FillGradient.IsValid())
		{
			return FillGradient.GetAverageColor();
		}
		else
		{
			return Style.GetFillColor();
		}
	}

	bool HasStroke() const { return Style.HasStroke(); }

	bool HasFill() const { return Style.HasFill(); }

	bool IsClosed() const { return bIsClosed; }

	int32 PolygonsNum() const { return Polygons.Num(); }

	bool IsClockwise() const { return bIsClockwise; }

	const FString& GetId() const { return Id; }

	const TArray<FSVGPathPolygon>& GetPolygons() const { return Polygons; }

	const FSVGStyle& GetStyle() const { return Style; }

	void SetStyle(const FSVGStyle& InStyle)
	{
		Style = InStyle;
	}

	void AddPolygon(const TArray<FVector>& InPoints);

	void ApplyFillRule();

private:
	UPROPERTY()
	FString Id;

	UPROPERTY()
	TArray<FSVGPathPolygon> Polygons;

	UPROPERTY()
	TArray<TObjectPtr<USplineComponent>> PolygonSplines;

	UPROPERTY()
	FSVGStyle Style;

	UPROPERTY()
	FSVGGradient FillGradient;

	UPROPERTY()
	FSVGGradient StrokeGradient;

	bool bIsClosed;

	bool bIsClockwise;

	bool bShapesHaveBeenGenerated;
};
