// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SVGImporter : ModuleRules
{
	public SVGImporter(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"GeometryFramework",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"GeometryAlgorithms",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"Projects",
				"Slate",
				"SlateCore",
			});

		// todo: there are some things we can move to editor module (e.g. SVGHelperLibrary)
		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"GeometryScriptingEditor",
					"LevelEditor",
					"UnrealEd"
				});
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Nanosvg");
	}
}
