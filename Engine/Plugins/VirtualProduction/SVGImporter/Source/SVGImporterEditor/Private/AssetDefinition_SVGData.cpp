// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SVGData.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorReimportHandler.h"
#include "SVGData.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_SVGData"

FText UAssetDefinition_SVGData::GetAssetDisplayName() const
{
	return LOCTEXT("SVGData", "SVG Data");
}

TSoftClassPtr<UObject> UAssetDefinition_SVGData::GetAssetClass() const
{
	return USVGData::StaticClass();
}

FLinearColor UAssetDefinition_SVGData::GetAssetColor() const
{
	return FColor(25, 178, 255);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SVGData::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Misc};
	return Categories;
}

void UAssetDefinition_SVGData::ExecuteReimportSVG(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	 
	for (USVGData* const LoadedSVG : Context->LoadSelectedObjects<USVGData>())
	{
		if (LoadedSVG)
		{
			LoadedSVG->Reset();
			
			constexpr bool bAskForNewFileIfMissing = true;
			FReimportManager::Instance()->Reimport(LoadedSVG, bAskForNewFileIfMissing);
		}
	}
}

namespace MenuExtension_SVGData
{
	// Self-registering callback to add menu items for SVG Data Actions
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
 		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
 		{
 			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME); 
 			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USVGData::StaticClass());

 				// The menu section to be customized with SVG Actions
				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				{
					Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						// SVG Data Reimport Action
						{
							const TAttribute<FText> Label = LOCTEXT("SVG_Reimport", "Reimport");
							const TAttribute<FText> ToolTip = LOCTEXT("SVG_ReimportTooltip", "Reimport SVG file");
							const FSlateIcon Icon = FSlateIcon();

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&UAssetDefinition_SVGData::ExecuteReimportSVG);
							InSection.AddMenuEntry("ReimportSVG", Label, ToolTip, Icon, UIAction);
						}
					}));
				}
 			}
 		}));
	});
}

#undef LOCTEXT_NAMESPACE
