// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_SVGData.generated.h"

struct FToolMenuContext;

UCLASS()
class UAssetDefinition_SVGData : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	/** Menu Extension statics */
	static void ExecuteReimportSVG(const FToolMenuContext& InContext);

	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ End UAssetDefinition
};
