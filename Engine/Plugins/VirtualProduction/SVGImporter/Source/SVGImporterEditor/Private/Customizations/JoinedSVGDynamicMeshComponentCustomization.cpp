// Copyright Epic Games, Inc. All Rights Reserved.

#include "JoinedSVGDynamicMeshComponentCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ProceduralMeshes/JoinedSVGDynamicMeshComponent.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

FJoinedSVGDynamicMeshComponentCustomization::~FJoinedSVGDynamicMeshComponentCustomization()
{
	DetailLayoutBuilderWeak.Reset();
}

void FJoinedSVGDynamicMeshComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// We want the SVG category before the Rendering one
	IDetailCategoryBuilder& SVGCategory = InDetailBuilder.EditCategory(TEXT("SVG"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	const FName ColoringPropertyName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, Coloring);
	const FName ShapeColorPropertyName = GET_MEMBER_NAME_CHECKED(FSVGShapeParameters, Color);
	const FName SVGLitPropertyName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, bSVGIsUnlit);
	const FName ShapeParametersListPropertyName = GET_MEMBER_NAME_CHECKED(UJoinedSVGDynamicMeshComponent, ShapeParametersList);

	const TSharedRef<IPropertyHandle> ColoringProperty = InDetailBuilder.GetProperty(ColoringPropertyName);
	uint8 MeshColoringInt;
	ColoringProperty->GetValue(MeshColoringInt);

	const FSimpleDelegate MeshColoringChangedDelegate = FSimpleDelegate::CreateSP(this, &FJoinedSVGDynamicMeshComponentCustomization::OnMeshColoringChanged);
	ColoringProperty->SetOnPropertyValueChanged(MeshColoringChangedDelegate);

	// Hide the list we want to customize, and the other properties so that we can force their order
	InDetailBuilder.HideProperty(ShapeParametersListPropertyName);
	InDetailBuilder.HideProperty(ColoringPropertyName);
	InDetailBuilder.HideProperty(SVGLitPropertyName);

	SVGCategory.AddProperty(ColoringPropertyName);
	SVGCategory.AddProperty(SVGLitPropertyName);

	const EJoinedSVGMeshColoring MeshColoring = static_cast<EJoinedSVGMeshColoring>(MeshColoringInt);
	if (MeshColoring == EJoinedSVGMeshColoring::SingleColor)
	{
		return;
	}

	const TSharedRef<IPropertyHandle> ShapeParametersListProperty = InDetailBuilder.GetProperty(ShapeParametersListPropertyName);
	const TSharedPtr<IPropertyHandleSet> ParametersListSetProperty = ShapeParametersListProperty->AsSet();

	uint32 NumElements;
	ParametersListSetProperty->GetNumElements(NumElements);

	for (uint32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
	{
		const TSharedRef<IPropertyHandle> ShapeParametersProperty = ParametersListSetProperty->GetElement(ElementIndex);
		const TSharedPtr<IPropertyHandle> ShapeColorProperty = ShapeParametersProperty->GetChildHandle(ShapeColorPropertyName);

		SVGCategory.AddProperty(ShapeColorProperty);
	}
}

void FJoinedSVGDynamicMeshComponentCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailLayoutBuilderWeak = InDetailBuilder;
	FJoinedSVGDynamicMeshComponentCustomization::CustomizeDetails(*InDetailBuilder);
}

void FJoinedSVGDynamicMeshComponentCustomization::OnMeshColoringChanged() const
{
	if (DetailLayoutBuilderWeak.IsValid())
	{
		if (IDetailLayoutBuilder* DetailLayoutBuilder = DetailLayoutBuilderWeak.Pin().Get())
		{
			DetailLayoutBuilder->ForceRefreshDetails();
		}
	}
}

TSharedRef<IPropertyTypeCustomization> FSVGShapeParametersColorDetails::MakeInstance()
{
	return MakeShared<FSVGShapeParametersColorDetails>();
}

void FSVGShapeParametersColorDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	FColorStructCustomization::CustomizeHeader(InStructPropertyHandle, InHeaderRow, InStructCustomizationUtils);

	if (const TSharedPtr<IPropertyHandle>& ShapeParametersHandle = InStructPropertyHandle->GetParentHandle())
	{
		if (const TSharedPtr<IPropertyHandle> ShapeNameHandle = ShapeParametersHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSVGShapeParameters, ShapeName)))
		{
			FString ShapeName;
			ShapeNameHandle->GetValue(ShapeName);

			InHeaderRow
			.NameContent()
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ShapeName))
			];
		}
	}
}
