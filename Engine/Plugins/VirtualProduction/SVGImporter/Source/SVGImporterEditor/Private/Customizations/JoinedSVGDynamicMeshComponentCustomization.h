// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Customizations/ColorStructCustomization.h"

class IDetailLayoutBuilder;

/**
 * Customizing JoinedSVGDynamicMeshComponent
 */
class FJoinedSVGDynamicMeshComponentCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FJoinedSVGDynamicMeshComponentCustomization>();
	}


	~FJoinedSVGDynamicMeshComponentCustomization();

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	void OnMeshColoringChanged() const;

	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilderWeak;
};

/**
 * Property identifier for the Color property found within FSVGShapeParameters
 */
class FSVGShapeParametersColorPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		static const TArray<FName> PropertyMetaTags = { 
			TEXT("SVGShapeParamColor")
		};

		for (const FName& MetaTag : PropertyMetaTags)
		{
			if (InPropertyHandle.HasMetaData(MetaTag))
			{
				return true;
			}
		}

		return false;
	}
};

/**
 * Customization for the color in FSVGShapeParameters.
 * This combined with the JoinedSVGDynamicMeshComponent customization above allows us to
 * Properly expose the color to RC, despite it being in a struct handled by a TSet
 */
class FSVGShapeParametersColorDetails : public FColorStructCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	//~End IPropertyTypeCustomization
};
