// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGCategoryTypeCustomization.h"
#include "DetailLayoutBuilder.h"

void FSVGCategoryTypeCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// We want the SVG category before the Rendering one
	InDetailBuilder.EditCategory(TEXT("SVG"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);
}
