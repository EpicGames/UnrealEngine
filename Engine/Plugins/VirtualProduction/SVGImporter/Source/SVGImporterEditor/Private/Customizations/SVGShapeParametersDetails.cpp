// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGShapeParametersDetails.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "ProceduralMeshes/JoinedSVGDynamicMeshComponent.h"
#include "Widgets/Layout/SBox.h"

TSharedRef<IPropertyTypeCustomization> FSVGShapeParametersDetails::MakeInstance()
{
	return MakeShared<FSVGShapeParametersDetails>();
}

void FSVGShapeParametersDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle>& NamePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSVGShapeParameters, ShapeName));

	InHeaderRow
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		NamePropertyHandle->CreatePropertyValueWidget()
	]
	.ValueContent()
	.HAlign(HAlign_Left)
	[
		// Assign the value content in CustomizeChildren, so we can use StructBuilder to easily create the Color Widget
		SAssignNew(HeaderValueContentBox, SBox)
	];
}

void FSVGShapeParametersDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle>& ColorPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSVGShapeParameters, Color));

	if (HeaderValueContentBox)
	{
		HeaderValueContentBox->SetContent
		(
			InStructBuilder.GenerateStructValueWidget(ColorPropertyHandle.ToSharedRef())
		);
	}
}
