// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SVGActorFactory.h"
#include "EngineAnalytics.h"
#include "LevelEditorViewport.h"
#include "SVGActor.h"
#include "SVGData.h"
#include "Subsystems/PlacementSubsystem.h"

USVGActorFactory::USVGActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NewActorClass = ASVGActor::StaticClass();
}

bool USVGActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	return (AssetData.IsValid() && AssetData.IsInstanceOf(USVGData::StaticClass()));
}

AActor* USVGActorFactory::GetDefaultActor(const FAssetData& AssetData)
{
	return NewActorClass->GetDefaultObject<ASVGActor>();
}

AActor* USVGActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	ASVGActor* SVGActor;
	{
		FSVGActorInitGuard InitGuard;
		SVGActor = Cast<ASVGActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));
	}

	SVGActor->SVGData = Cast<USVGData>(InAsset);

	if (InSpawnParams.bTemporaryEditorActor)
	{
		SVGActor->RenderMode = ESVGRenderMode::Texture2D;
	}
	else
	{
		SVGActor->RenderMode = ESVGRenderMode::DynamicMesh3D;
	}

	SVGActor->Initialize();
	return SVGActor;
}

void USVGActorFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InHandle, InPlacementInfo, InPlacementOptions);

	if (!InPlacementOptions.bIsCreatingPreviewElements && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("ToolClass"), GetNameSafe(GetClass()));
		Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(NewActorClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}
