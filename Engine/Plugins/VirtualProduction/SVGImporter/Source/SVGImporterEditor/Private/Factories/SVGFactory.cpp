// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGFactory.h"
#include "Editor.h"
#include "SVGData.h"
#include "SVGImporterEditorUtils.h"
#include "Subsystems/ImportSubsystem.h"

namespace UE::SVGImporterEditor::Private
{
	// The DPI is only used for unit conversion
	constexpr static float SVG_DPI = 96.0f;

	// Bytes per pixel (RGBA)
	constexpr static int32 SVG_BPP = 4;
}

USVGFactory::USVGFactory(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Formats.Add(TEXT("svg;SVG Data"));
	SupportedClass = USVGData::StaticClass();
	bCreateNew = false; // turned off for import
	bEditAfterNew = false; // turned off for import
	bEditorImport = true;
	bText = true;
}

UObject* USVGFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	USVGData* NewObjectAsset = NewObject<USVGData>(InParent, InClass, InName, InFlags | RF_Transactional);
	return NewObjectAsset;
}

UObject* USVGFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, const TCHAR* InType, const TCHAR*& InBuffer, const TCHAR* InBufferEnd, FFeedbackContext* InWarn)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, InType);

	// If class type or extension doesn't match, return
	if (InClass != USVGData::StaticClass() || FCString::Stricmp(InType, TEXT("svg")) != 0)
	{
		return nullptr;
	}

	if (USVGData* SVGData = FSVGImporterEditorUtils::CreateSVGDataFromTextBuffer(InBuffer, InParent, InName, InFlags, CurrentFilename))
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, SVGData);
		return SVGData;
	}

	return nullptr;
}

bool USVGFactory::FactoryCanImport(const FString& InFilename)
{
	return FPaths::GetExtension(InFilename).Equals(TEXT("svg"));
}
