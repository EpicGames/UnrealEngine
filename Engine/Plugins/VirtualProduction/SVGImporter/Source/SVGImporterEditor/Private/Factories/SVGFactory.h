// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "SVGFactory.generated.h"

UCLASS()
class USVGFactory : public UFactory
{
	GENERATED_BODY()

public:
	//~ Begin UFactory
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	virtual bool FactoryCanImport(const FString& InFilename) override;
	//~ End UFactory

protected:
	USVGFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UFactory
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, const TCHAR* InType, const TCHAR*& InBuffer, const TCHAR* InBufferEnd, FFeedbackContext* InWarn) override;
	//~ End UFactory
};
