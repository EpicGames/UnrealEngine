// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGReimportFactory.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/FileHelper.h"
#include "SVGData.h"
#include "SVGImporterEditorUtils.h"

bool USVGReimportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (const USVGData* ObjectAsset = Cast<USVGData>(Obj))
    {
        OutFilenames.Add(UAssetImportData::ResolveImportFilename(ObjectAsset->SourceFilePath, ObjectAsset->GetOutermost()));
        return true;
    }
    return false;
}

void USVGReimportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
    USVGData* ObjectAsset = Cast<USVGData>(Obj);
    if (ObjectAsset && ensure(NewReimportPaths.Num() == 1))
    {
        ObjectAsset->SourceFilePath = UAssetImportData::SanitizeImportFilename(NewReimportPaths[0], ObjectAsset->GetOutermost());
    }
}

EReimportResult::Type USVGReimportFactory::Reimport(UObject* Obj)
{
    USVGData* SVGDataAsset = Cast<USVGData>(Obj);
    if (!SVGDataAsset)
    {
        return EReimportResult::Failed;
    }

    const FString Filename = UAssetImportData::ResolveImportFilename(SVGDataAsset->SourceFilePath, SVGDataAsset->GetOutermost());
    if (!FPaths::GetExtension(Filename).Equals(TEXT("svg")))
    {
        return EReimportResult::Failed;
    }

    CurrentFilename = Filename;
    FString SVGBufferData;
    if (FFileHelper::LoadFileToString(SVGBufferData, *CurrentFilename))
    {
        const TCHAR* Ptr = *SVGBufferData;
        SVGDataAsset->Modify();

        SVGDataAsset->MarkPackageDirty();

        const bool bRefreshSuccess = FSVGImporterEditorUtils::RefreshSVGDataFromTextBuffer(SVGDataAsset, Ptr);
        if (bRefreshSuccess)
        {
            // save the source file path and timestamp
            SVGDataAsset->SourceFilePath = UAssetImportData::SanitizeImportFilename(CurrentFilename, SVGDataAsset->GetOutermost());
            return EReimportResult::Succeeded;
        }
    }
    
    return EReimportResult::Failed;
}