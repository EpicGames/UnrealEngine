// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "SVGFactory.h"
#include "SVGReimportFactory.generated.h"

UCLASS()
class USVGReimportFactory : public USVGFactory, public FReimportHandler
{
    GENERATED_BODY()

    // Begin FReimportHandler interface
    virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
    virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
    virtual EReimportResult::Type Reimport(UObject* Obj) override;
    // End FReimportHandler interface
};