// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGActorContextMenu.h"
#include "ISVGImporterEditorModule.h"
#include "SVGActor.h"
#include "SVGImporterUtils.h"
#include "SVGShapesParentActor.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "SVGActorContextMenu"

void FSVGActorContextMenu::AddSVGActorMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& InSelectedActors)
{
	if (!CanAddSVGMenuEntries(InSelectedActors))
	{
		return;
	}

	FToolMenuSection& MenuSection = InToolMenu->FindOrAddSection(ISVGImporterEditorModule::Get().GetSVGImporterMenuCategoryName(), LOCTEXT("SVGActorActionsMenuHeading", "SVG Importer"));

	MenuSection.AddMenuEntry(
		TEXT("SVGActor_SplitSVGActor"),
		LOCTEXT("SplitSVGActorMenuEntry", "Split SVG Actor"),
		LOCTEXT("SplitSVGActorMenuEntryTooltip", "Split the SVG Actor, distributing its shapes over multiple Shape Actors, one per shape. A main attachment Actor will be created to organize the Shape Actors."),
		FSlateIcon(ISVGImporterEditorModule::Get().GetStyleName(), "ClassIcon.SVGActor"),
		FUIAction(
			FExecuteAction::CreateStatic(&FSVGActorContextMenu::ExecuteSplitAction, InSelectedActors),
			FCanExecuteAction::CreateStatic(&FSVGActorContextMenu::CanExecuteSVGActorAction, InSelectedActors),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FSVGActorContextMenu::CanExecuteSVGActorAction, InSelectedActors))
	);

	MenuSection.AddMenuEntry(
		TEXT("SVGActor_ConsolidateSVGActor"),
		LOCTEXT("ConsolidateSVGActorMenuEntry", "Consolidate SVG Actor(s)"),
		LOCTEXT("ConsolidateSVGActorMenuEntryTooltip", "Consolidates the SVG or SVG Shape Parents Actor(s) onto a single actor with a single Dynamic Mesh Component."),
		FSlateIcon(ISVGImporterEditorModule::Get().GetStyleName(), "ClassIcon.SVGActor"),
		FUIAction(
			FExecuteAction::CreateStatic(&FSVGActorContextMenu::ExecuteConsolidateAction, InSelectedActors),
			FCanExecuteAction::CreateStatic(&FSVGActorContextMenu::CanExecuteConsolidateAction, InSelectedActors),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FSVGActorContextMenu::CanExecuteConsolidateAction, InSelectedActors))
	);

	MenuSection.AddMenuEntry(
		TEXT("SVGActor_JoinSVGShapes"),
		LOCTEXT("JoinSVGShapesMenuEntry", "Join SVG Shapes"),
		LOCTEXT("JoinSVGShapesMenuEntryTooltip", "Join SVG dynamic meshes onto one, generating a new dedicated actor."),
		FSlateIcon(ISVGImporterEditorModule::Get().GetStyleName(), "ClassIcon.SVGActor"),
		FUIAction(
			FExecuteAction::CreateStatic(&FSVGActorContextMenu::ExecuteJoinAction, InSelectedActors),
			FCanExecuteAction::CreateStatic(&FSVGActorContextMenu::CanExecuteJoinAction, InSelectedActors),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FSVGActorContextMenu::CanExecuteJoinAction, InSelectedActors))
	);
}

void FSVGActorContextMenu::ExecuteSplitAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (ASVGActor* SVGActor = Cast<ASVGActor>(Actor.Get()))
			{
				FSVGImporterUtils::SplitSVGActor(SVGActor);
			}
		}
	}
}

bool FSVGActorContextMenu::CanExecuteSVGActorAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (!Actor->IsA<ASVGActor>())
			{
				return false;
			}
		}
	}

	return true;
}

void FSVGActorContextMenu::ExecuteJoinAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	TArray<ASVGDynamicMeshesContainerActor*> ShapesActors;

	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (ASVGDynamicMeshesContainerActor* ShapesActor = Cast<ASVGDynamicMeshesContainerActor>(Actor))
			{
				ShapesActors.Add(ShapesActor);
			}
		}
	}

	FSVGImporterUtils::JoinSVGDynamicMeshOwners(ShapesActors);
}

bool FSVGActorContextMenu::CanExecuteJoinAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	// At least 2 actors in order to join them
	if (InActors.Num() < 2)
	{
		return false;
	}

	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (!Actor->IsA<ASVGDynamicMeshesContainerActor>())
			{
				return false;
			}
		}
	}

	return true;
}

bool FSVGActorContextMenu::CanAddSVGMenuEntries(const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	if (InActors.IsEmpty())
	{
		return false;
	}

	return CanExecuteJoinAction(InActors) || CanExecuteConsolidateAction(InActors);
}

void FSVGActorContextMenu::ExecuteConsolidateAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (ASVGActor* SVGActor = Cast<ASVGActor>(Actor.Get()))
			{
				FSVGImporterUtils::ConsolidateSVGActor(SVGActor);
			}
			else if (ASVGShapesParentActor* SVGShapesParentActor = Cast<ASVGShapesParentActor>(Actor.Get()))
			{
				FSVGImporterUtils::JoinSVGDynamicMeshOwners({ SVGShapesParentActor });
			}
		}
	}
}

bool FSVGActorContextMenu::CanExecuteConsolidateAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	if (CanExecuteSVGActorAction(InActors))
	{
		return true;
	}

	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (!Actor->IsA<ASVGShapesParentActor>())
			{
				return false;
			}
		}
	}

	return true;
}
#undef LOCTEXT_NAMESPACE
