// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class UToolMenu;

class FSVGActorContextMenu
{
public:
	static void AddSVGActorMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& InSelectedActors);

private:
	static bool CanAddSVGMenuEntries(const TSet<TWeakObjectPtr<AActor>>& InActors);

	static bool CanExecuteSVGActorAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static void ExecuteSplitAction(TSet<TWeakObjectPtr<AActor>> InActors);

	static void ExecuteConsolidateAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static bool CanExecuteConsolidateAction(TSet<TWeakObjectPtr<AActor>> InActors);

	static void ExecuteJoinAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static bool CanExecuteJoinAction(TSet<TWeakObjectPtr<AActor>> InActors);
};
