// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGActorEditorComponentVisualizer.h"
#include "EditorViewportClient.h"
#include "HitProxies.h"
#include "SVGActor.h"
#include "SVGActorEditorComponent.h"
#include "PrimitiveDrawingUtils.h"

float FSVGActorEditorComponentVisualizer::FillsExtrudeMin = 0.01f;
float FSVGActorEditorComponentVisualizer::FillsExtrudeMax = 20.0f;
float FSVGActorEditorComponentVisualizer::StrokesExtrudeMin = 0.01f;
float FSVGActorEditorComponentVisualizer::StrokesExtrudeMax = 20.0f;

struct HSVGActorExtrudeHitProxy : HComponentVisProxy
{
	DECLARE_HIT_PROXY();

	HSVGActorExtrudeHitProxy(const UActorComponent* InComponent, ASVGActor* InSVGActor)
		: HComponentVisProxy(InComponent, HPP_Wireframe)
		, SVGActorWeak(InSVGActor)
	{ }

	TWeakObjectPtr<ASVGActor> SVGActorWeak;
};

IMPLEMENT_HIT_PROXY(HSVGActorExtrudeHitProxy, HComponentVisProxy)

FSVGActorEditorComponentVisualizer::FSVGActorEditorComponentVisualizer()
{
	bIsExtruding = false;
	UpdateMinMaxExtrudeValues();
}

void FSVGActorEditorComponentVisualizer::DrawVisualization(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	constexpr static float ExtrudeHandleSize = 10.0f;

	const USVGActorEditorComponent* SVGActorEditorComponent = Cast<USVGActorEditorComponent>(InComponent);
	if (!SVGActorEditorComponent)
	{
		InPDI->SetHitProxy(nullptr);
		EndEditing();
		return;
	}

	if (ASVGActor* SVGActor = Cast<ASVGActor>(SVGActorEditorComponent->GetSVGActor()))
	{
		if (SVGActor->ExtrudeType == ESVGExtrudeType::None || SVGActor->RenderMode == ESVGRenderMode::Texture2D)
		{
			EndEditing();
		}
		else
		{
			SVGActorWeak = SVGActor;
			SVGEditorComponentWeak = SVGActor->GetSVGEditorComponent();

			if (SVGEditorComponentWeak.IsValid())
			{
				const FVector ProxyHandleLocation = GetExtrudeWidgetLocation();
				const FVector LineStartLocation = GetExtrudeSurfaceLocation();
				InPDI->DrawLine(LineStartLocation, ProxyHandleLocation, FLinearColor::White, SDPG_Foreground);

				InPDI->SetHitProxy(new HSVGActorExtrudeHitProxy(SVGActorEditorComponent, SVGActor));
				InPDI->DrawPoint(ProxyHandleLocation, FLinearColor::White, ExtrudeHandleSize, SDPG_Foreground);
				InPDI->SetHitProxy(nullptr);
			}
		}
	}
}

bool FSVGActorEditorComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	EndEditing();

	if (InClick.GetKey() == EKeys::LeftMouseButton && InVisProxy)
	{
		if (InVisProxy->IsA(HSVGActorExtrudeHitProxy::StaticGetType()))
		{
			if (const HSVGActorExtrudeHitProxy* const ExtrudeProxy = static_cast<HSVGActorExtrudeHitProxy*>(InVisProxy))
			{
				if (ASVGActor* SVGActor = ExtrudeProxy->SVGActorWeak.Get())
				{
					if (GEditor)
					{
						GEditor->SelectActor(SVGActor, true, false);
					}

					bIsExtruding = true;
					SVGActorWeak = SVGActor;
					SVGEditorComponentWeak = Cast<USVGActorEditorComponent>(const_cast<UActorComponent*>(ExtrudeProxy->Component.Get()));
					InViewportClient->SetWidgetMode(UE::Widget::WM_None);

					return true;
				}
			}
		}
	}

	return false;
}

bool FSVGActorEditorComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const
{
	if (bIsExtruding)
	{
		if (SVGActorWeak.IsValid())
		{
			OutLocation = GetExtrudeWidgetLocation();
			return true;
		}
	}

	return false;
}

bool FSVGActorEditorComponentVisualizer::HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDeltaTranslate, FRotator& InDeltaRotate, FVector& InDeltaScale)
{
	if (!GetEditedComponent() || !GetEditedComponent()->GetOwner())
	{
		EndEditing();
		return false;
	}

	if (!InDeltaTranslate.IsZero())
	{
		if (bIsExtruding)
		{
			if (ASVGActor* SVGActor = SVGActorWeak.Get())
			{
				SVGActor->Modify();

				float FillsExtrudeDepth = SVGActor->GetFillsExtrude() - (InDeltaTranslate.X / SVGActor->GetScale());
				FillsExtrudeDepth = FMath::Clamp(FillsExtrudeDepth, FillsExtrudeMin, FillsExtrudeMax);
				SVGActor->SetFillsExtrudeInteractive(FillsExtrudeDepth);

				float StrokesExtrudeDepth = SVGActor->GetStrokesExtrude() - (InDeltaTranslate.X / SVGActor->GetScale());
				StrokesExtrudeDepth = FMath::Clamp(StrokesExtrudeDepth, StrokesExtrudeMin, StrokesExtrudeMax);
				SVGActor->SetStrokesExtrudeInteractive(StrokesExtrudeDepth);

				return true;
			}
		}
	}

	EndEditing();
	return false;
}

void FSVGActorEditorComponentVisualizer::EndEditing()
{
	bIsExtruding = false;
}

void FSVGActorEditorComponentVisualizer::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	if (!bIsExtruding)
	{
		return;
	}

	if (ASVGActor* SVGActor = SVGActorWeak.Get())
	{
		SVGActor->Modify();

		const float ExtrudeDepth = FMath::Clamp(SVGActor->GetFillsExtrude(), FillsExtrudeMin, FillsExtrudeMax);
		SVGActor->SetFillsExtrude(ExtrudeDepth);

		const float StrokesExtrudeDepth = FMath::Clamp(SVGActor->GetStrokesExtrude(), StrokesExtrudeMin, StrokesExtrudeMax);
		SVGActor->SetStrokesExtrude(StrokesExtrudeDepth);
	}

	FComponentVisualizer::TrackingStopped(InViewportClient, bInDidMove);
}

UActorComponent* FSVGActorEditorComponentVisualizer::GetEditedComponent() const
{
	return SVGEditorComponentWeak.Get();
}

FVector FSVGActorEditorComponentVisualizer::GetExtrudeSurfaceLocation() const
{
	if (const ASVGActor* SVGActor = SVGActorWeak.Get())
	{
		float ExtrudeDepth = FMath::Max(SVGActor->GetFillsExtrude(), SVGActor->GetStrokesExtrude());

		if (SVGActor->ExtrudeType == ESVGExtrudeType::FrontBackMirror)
		{
			ExtrudeDepth *= 0.5f;
		}

		const FVector SurfaceOffset = -SVGActor->GetActorForwardVector() * ExtrudeDepth * SVGActor->GetScale();

		return SVGActor->GetActorLocation() + SurfaceOffset;
	}

	return {};
}

FVector FSVGActorEditorComponentVisualizer::GetExtrudeWidgetLocation() const
{
	constexpr static float ExtrudeHandleOffset = 10.0f;
	const FVector SurfaceLocation = GetExtrudeSurfaceLocation();

	if (const ASVGActor* SVGActor = SVGActorWeak.Get())
	{
		const float ExtrudeDepth = FMath::Max(SVGActor->GetFillsExtrude(), SVGActor->GetStrokesExtrude());
		FVector HandleOffset = SVGActor->GetActorForwardVector() * ExtrudeHandleOffset;

		if (ExtrudeDepth >= 0)
		{
			HandleOffset = -HandleOffset;
		}

		return SurfaceLocation + HandleOffset * SVGActor->GetScale();
	}

	return {};
}

void FSVGActorEditorComponentVisualizer::UpdateMinMaxExtrudeValues()
{
	// Get Min and Max Extrude values from Properties Metadata, to match what the UI allows

	if (const FProperty* FillsExtrudeProperty = ASVGActor::StaticClass()->FindPropertyByName(TEXT("FillsExtrude")))
	{
		if (FillsExtrudeProperty->HasMetaData("UIMin"))
		{
			FillsExtrudeMin = FCString::Atof(*FillsExtrudeProperty->GetMetaData("UIMin"));
		}

		if (FillsExtrudeProperty->HasMetaData("UIMax"))
		{
			FillsExtrudeMax = FCString::Atof(*FillsExtrudeProperty->GetMetaData("UIMax"));
		}
	}

	if (const FProperty* StrokesExtrudeProperty = ASVGActor::StaticClass()->FindPropertyByName(TEXT("StrokesExtrude")))
	{
		if (StrokesExtrudeProperty->HasMetaData("UIMin"))
		{
			StrokesExtrudeMin = FCString::Atof(*StrokesExtrudeProperty->GetMetaData("UIMin"));
		}

		if (StrokesExtrudeProperty->HasMetaData("UIMax"))
		{
			StrokesExtrudeMax = FCString::Atof(*StrokesExtrudeProperty->GetMetaData("UIMax"));
		}
	}
}
