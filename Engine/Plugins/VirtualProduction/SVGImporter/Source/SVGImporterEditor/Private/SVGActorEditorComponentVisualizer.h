// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

class ASVGActor;
class USVGActorEditorComponent;

class FSVGActorEditorComponentVisualizer : public FComponentVisualizer
{
public:
	static void UpdateMinMaxExtrudeValues();

	FSVGActorEditorComponentVisualizer();

	//~ Begin FComponentVisualizer
	virtual void DrawVisualization(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDeltaTranslate, FRotator& InDeltaRotate, FVector& InDeltaScale) override;
	virtual void EndEditing() override;
	virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) override;
	virtual UActorComponent* GetEditedComponent() const override;
	//~ End FComponentVisualizer

private:
	FVector GetExtrudeWidgetLocation() const;
	FVector GetExtrudeSurfaceLocation() const;

	static float FillsExtrudeMin;
	static float FillsExtrudeMax;
	static float StrokesExtrudeMin;
	static float StrokesExtrudeMax;

	TWeakObjectPtr<USVGActorEditorComponent> SVGEditorComponentWeak;
	TWeakObjectPtr<ASVGActor> SVGActorWeak;
	bool bIsExtruding = false;
};
