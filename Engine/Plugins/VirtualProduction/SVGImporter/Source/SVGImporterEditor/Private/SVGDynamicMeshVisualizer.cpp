// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGDynamicMeshVisualizer.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "PrimitiveDrawingUtils.h"

void FSVGDynamicMeshVisualizer::DrawVisualization(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	if (const USVGDynamicMeshComponent* SVGMeshComp = Cast<USVGDynamicMeshComponent>(InComponent))
	{
		// draw box to highlight bevel setting while in interactive mode
		if (SVGMeshComp->bIsBevelBeingEdited)
		{
			FVector BoxLocation = SVGMeshComp->GetComponentLocation() - FVector(SVGMeshComp->Bounds.BoxExtent.X, 0, 0) + FVector(SVGMeshComp->Bevel * 0.5f, 0, 0);

			if (SVGMeshComp->ExtrudeType == ESVGExtrudeType::FrontFaceOnly)
			{
				BoxLocation -= FVector(SVGMeshComp->Bounds.BoxExtent.X, 0, 0);
			}

			FBox Box;
			Box.Max = BoxLocation + FVector(SVGMeshComp->Bevel * 0.5f, SVGMeshComp->Bounds.BoxExtent.Y, SVGMeshComp->Bounds.BoxExtent.Z);
			Box.Min = BoxLocation - FVector(SVGMeshComp->Bevel * 0.5f, SVGMeshComp->Bounds.BoxExtent.Y, SVGMeshComp->Bounds.BoxExtent.Z);

			DrawWireBox(InPDI, Box, FColor::Green, SDPG_Foreground);
		}
		// in other cases, if the SVG mesh is selected, draw a box around it
		else if (SVGMeshComp->IsSelectedInEditor())
		{
			const FBox Bounds(SVGMeshComp->Bounds.GetBox());
			DrawWireBox(InPDI, Bounds, FColor::Red, SDPG_Foreground);
		}
	}
}
