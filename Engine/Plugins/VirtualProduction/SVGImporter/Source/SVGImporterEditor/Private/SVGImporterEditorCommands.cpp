// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterEditorCommands.h"
#include "ISVGImporterEditorModule.h"

#define LOCTEXT_NAMESPACE "AvaSVGEditorCommands"

const FSVGImporterEditorCommands& FSVGImporterEditorCommands::GetExternal()
{
	if (!IsRegistered())
	{
		Register();
	}

	return Get();
}

const FSVGImporterEditorCommands& FSVGImporterEditorCommands::GetInternal()
{
	return Get();
}

FSVGImporterEditorCommands::FSVGImporterEditorCommands()
	: TCommands<FSVGImporterEditorCommands>(
		TEXT("SVGImporterEditor")
		, LOCTEXT("SVGImporterEditor", "SVG Importer Editor")
		, NAME_None
		, ISVGImporterEditorModule::Get().GetStyleName()
	)
{
}

void FSVGImporterEditorCommands::RegisterCommands()
{
	UI_COMMAND(SpawnSVGActor
		, "SVG Actor"
		, "Create a new SVG Actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
