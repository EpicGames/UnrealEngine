// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterEditorModule.h"
#include "AssetSelection.h"
#include "AssetToolsModule.h"
#include "ComponentVisualizer.h"
#include "Editor/UnrealEdEngine.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "Interfaces/IPluginManager.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "PropertyEditorModule.h"
#include "SVGActor.h"
#include "SVGActorContextMenu.h"
#include "SVGActorEditorComponent.h"
#include "SVGActorEditorComponentVisualizer.h"
#include "SVGData.h"
#include "SVGDynamicMeshVisualizer.h"
#include "SVGImporter.h"
#include "SVGImporterEditorCommands.h"
#include "SVGImporterEditorUtils.h"
#include "SVGParsingUtils.h"
#include "SVGThumbnailRenderer.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "Customizations/JoinedSVGDynamicMeshComponentCustomization.h"
#include "Customizations/SVGCategoryTypeCustomization.h"
#include "Customizations/SVGShapeParametersDetails.h"

#define LOCTEXT_NAMESPACE "SVGImporterEditorModule"

DEFINE_LOG_CATEGORY(SVGImporterEditorLog);

namespace UE::SVGImporterEditor::Private
{
	static const FName SVGFromClipboardEntryName = TEXT("SVGFromClipboard");
	static const FText SVGFromClipboardLabel = LOCTEXT("SVGFromClipboardLabel", "Create SVG Actor from clipboard");
	static const FText SVGFromClipboardDescription = LOCTEXT("SVGFromClipboardDescription", "Creates an SVG Actor from the clipboard, if clipboard content is a valid SVG source");

	static TSet<FName> SVGCategoryTypes
		{
			TEXT("SVGActor")
		  , TEXT("SVGShapeActor")
		  , TEXT("SVGJoinedShapesActor")
		  , TEXT("SVGDynamicMeshComponent")
	  };
}

FName FSVGImporterEditorModule::SVGImporterCategoryName = TEXT("SVGImporter");;

void FSVGImporterEditorModule::StartupModule()
{
	FSVGImporterEditorCommands::Register();

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	AssetTools.RegisterAdvancedAssetCategory(GetSVGImporterMenuCategoryName(), LOCTEXT("SVGImporterCategoryName", "SVGImporter"));

	UThumbnailManager::Get().RegisterCustomRenderer(USVGData::StaticClass(), USVGThumbnailRenderer::StaticClass());

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FSVGImporterEditorModule::PostEngineInit);

	RegisterContextMenuExtender();

	FSVGImporterModule::Get().OnDefaultSVGDataRequested.BindRaw(this, &FSVGImporterEditorModule::OnDefaultSVGDataRequested);

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("SVGShapeParameters", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSVGShapeParametersDetails::MakeInstance));

	for (const FName Type : UE::SVGImporterEditor::Private::SVGCategoryTypes)
	{
		PropertyModule.RegisterCustomClassLayout(Type, FOnGetDetailCustomizationInstance::CreateStatic(&FSVGCategoryTypeCustomization::MakeInstance));
	}

	const TSharedRef<FSVGShapeParametersColorPropertyTypeIdentifier> ColorPropertyTypeIdentifier = MakeShared<FSVGShapeParametersColorPropertyTypeIdentifier>();

	PropertyModule.RegisterCustomPropertyTypeLayout(TBaseStructure<FLinearColor>::Get()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(
		&FSVGShapeParametersColorDetails::MakeInstance), ColorPropertyTypeIdentifier);

	PropertyModule.RegisterCustomClassLayout("JoinedSVGDynamicMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FJoinedSVGDynamicMeshComponentCustomization::MakeInstance));
}

void FSVGImporterEditorModule::ShutdownModule()
{
	FSVGImporterEditorCommands::Unregister();

	if (FSVGImporterModule::Get().OnDefaultSVGDataRequested.IsBoundToObject(this))
	{
		FSVGImporterModule::Get().OnDefaultSVGDataRequested.Unbind();
	}
}

void FSVGImporterEditorModule::AddSVGActorFromClipboardMenuEntry(UToolMenu* InMenu)
{
	using namespace UE::SVGImporterEditor::Private;

	const ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>();
	if (!LevelEditorContext)
	{
		return;
	}

	const TSharedPtr<ILevelEditor>& LevelEditor = LevelEditorContext->LevelEditor.Pin();
	if (!LevelEditor)
	{
		return;
	}

	const UWorld* World = LevelEditor->GetWorld();
	if (!World)
	{
		return;
	}

	ULevel* CurrentLevel = World->GetCurrentLevel();
	if (!CurrentLevel)
	{
		return;
	}

	if (CanCreateSVGActorFromClipboard(CurrentLevel))
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection(GetMenuCategoryName(), LOCTEXT("SVGImporterHeading", "SVG Importer"));

		Section.AddMenuEntry(SVGFromClipboardEntryName
			, SVGFromClipboardLabel
			,SVGFromClipboardDescription
			, FSlateIcon()
			,CreatePasteSVGActorAction(CurrentLevel));
	}
}

void FSVGImporterEditorModule::AddSVGActorEntriesToLevelEditorContextMenu(UToolMenu* InMenu)
{
	if (const ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>())
	{
		TArray<UObject*> AllSelected = LevelEditorContext->CurrentSelection->GetSelectedObjects(AActor::StaticClass());

		if (AllSelected.IsEmpty())
		{
			return;
		}

		TSet<TWeakObjectPtr<AActor>> SelectedActors;
		for (UObject* ActorAsObject : AllSelected)
		{
			if (AActor* SelectedActor = Cast<AActor>(ActorAsObject))
			{
				SelectedActors.Add(SelectedActor);
			}
		}

		if (SelectedActors.IsEmpty())
		{
			return;
		}

		FSVGActorContextMenu::AddSVGActorMenuEntries(InMenu, SelectedActors);
	}
}

void FSVGImporterEditorModule::PostEngineInit()
{
	if (GUnrealEd)
	{
		const TSharedRef<FComponentVisualizer> SVGDynMeshVisualizer = MakeShared<FSVGDynamicMeshVisualizer>();
		GUnrealEd->RegisterComponentVisualizer(USVGDynamicMeshComponent::StaticClass()->GetFName(), SVGDynMeshVisualizer);

		const TSharedRef<FComponentVisualizer> SVGActorEditorComponentVisualizer = MakeShared<FSVGActorEditorComponentVisualizer>();
		GUnrealEd->RegisterComponentVisualizer(USVGActorEditorComponent::StaticClass()->GetFName(), SVGActorEditorComponentVisualizer);
	}

	// Register SVG category for needed types
	if (FSlateApplication::IsInitialized())
	{
		static const FName PropertyEditor("PropertyEditor");
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

		for (const FName Type : UE::SVGImporterEditor::Private::SVGCategoryTypes)
		{
			const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(Type, TEXT("SVG"), LOCTEXT("SVG", "SVG"));
			Section->AddCategory(TEXT("SVG"));
		}
	}
}

USVGData* FSVGImporterEditorModule::OnDefaultSVGDataRequested()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	FString ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));
	const FString FileToImport = FPaths::Combine(ContentRootDir, TEXT("Icons/DefaultSVG.svg"));

	FString SVGText;
	FFileHelper::LoadFileToString(SVGText, *FileToImport);

	if (SVGText.IsEmpty())
	{
		return nullptr;
	}

	USVGData* SVGData = FSVGImporterEditorUtils::CreateSVGDataFromTextBuffer(SVGText, GetTransientPackage(), TEXT("SVGData"), RF_Transactional | RF_Transient);

	return SVGData;
}

FUIAction FSVGImporterEditorModule::CreatePasteSVGActorAction(ULevel* InLevel)
{
	FUIAction PasteFromClipboardAction;
	PasteFromClipboardAction.ExecuteAction = FExecuteAction::CreateStatic(&FSVGImporterEditorModule::CreateSVGActorFromClipboard, InLevel);
	PasteFromClipboardAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FSVGImporterEditorModule::CanCreateSVGActorFromClipboard, InLevel);

	return PasteFromClipboardAction;
}

void FSVGImporterEditorModule::RegisterContextMenuExtender()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	using namespace UE::SVGImporterEditor::Private;
	if (UToolMenu* EmptySelectionMenu = ToolMenus->ExtendMenu("LevelEditor.EmptySelectionContextMenu"))
	{
		EmptySelectionMenu->AddDynamicSection(GetMenuCategoryName(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			AddSVGActorFromClipboardMenuEntry(InMenu);
		}));
	}

	if (UToolMenu* ActorContextMenu = ToolMenus->ExtendMenu("LevelEditor.ActorContextMenu"))
	{
		ActorContextMenu->AddDynamicSection(GetMenuCategoryName(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			AddSVGActorFromClipboardMenuEntry(InMenu);
			AddSVGActorEntriesToLevelEditorContextMenu(InMenu);
		}));
	}
}

void FSVGImporterEditorModule::AddSVGActorMenuEntries(UToolMenu* InMenu, TSet<TWeakObjectPtr<AActor>> InActors)
{
	FSVGActorContextMenu::AddSVGActorMenuEntries(InMenu, InActors);
}

void FSVGImporterEditorModule::CreateSVGActorFromClipboard(ULevel* InLevel)
{
	if (!InLevel)
	{
		return;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	if (!ClipboardContent.IsEmpty())
	{
		const FName SVGDataUniqueName = MakeUniqueObjectName(GetTransientPackage(), USVGData::StaticClass(), FName(TEXT("SVGData")));
		USVGData* SVGData = FSVGImporterEditorUtils::CreateSVGDataFromTextBuffer(ClipboardContent, GetTransientPackage(), SVGDataUniqueName, RF_Transactional | RF_Transient);
		if (!SVGData)
		{
			return;
		}

		if (ASVGActor* NewSVGActor = Cast<ASVGActor>(FActorFactoryAssetProxy::AddActorForAsset(SVGData)))
		{
			// We change the object outer so that SVGData belongs to SVGActor
			SVGData->Rename(nullptr, NewSVGActor);
			SVGData->ClearFlags(RF_Transient);
		}
	}
}

bool FSVGImporterEditorModule::CanCreateSVGActorFromClipboard(ULevel* InLevel)
{
	if (!InLevel)
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FSVGParsingUtils::IsValidSVGString(ClipboardContent);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FSVGImporterEditorModule, SVGImporterEditor)
