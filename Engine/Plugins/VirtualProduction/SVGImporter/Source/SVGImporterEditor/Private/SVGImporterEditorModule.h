// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISVGImporterEditorModule.h"
#include "SVGData.h"
#include "SVGImporterEditorStyle.h"
#include "Toolkits/AssetEditorToolkit.h"

class ULevel;

DECLARE_LOG_CATEGORY_EXTERN(SVGImporterEditorLog, Log, All);

class FSVGImporterEditorModule : public ISVGImporterEditorModule
{
public:
	static void AddSVGActorFromClipboardMenuEntry(UToolMenu* InMenu);
	static void AddSVGActorEntriesToLevelEditorContextMenu(UToolMenu* InMenu);

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface


	//~ Begin ISVGImporterEditorModule
	virtual const FName GetStyleName() const override
	{
		return FSVGImporterEditorStyle::Get().GetStyleSetName();
	}

	virtual FName GetSVGImporterMenuCategoryName() const override
	{
		return GetMenuCategoryName();
	}

	virtual void AddSVGActorMenuEntries(UToolMenu* InMenu, TSet<TWeakObjectPtr<AActor>> InActors) override;
	//~ Begin ISVGImporterEditorModule

protected:
	static void CreateSVGActorFromClipboard(ULevel* InLevel);
	static void RegisterContextMenuExtender();
	static bool CanCreateSVGActorFromClipboard(ULevel* InLevel);
	static FUIAction CreatePasteSVGActorAction(ULevel* InLevel);
	static FName GetMenuCategoryName() { return SVGImporterCategoryName; }

	void PostEngineInit();

	USVGData* OnDefaultSVGDataRequested();

	static FName SVGImporterCategoryName;
};
