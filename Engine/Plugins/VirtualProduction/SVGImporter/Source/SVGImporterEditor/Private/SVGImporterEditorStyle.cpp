// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FSVGImporterEditorStyle::FSVGImporterEditorStyle()
	: FSlateStyleSet(TEXT("SVGImporterEditor"))
{
	const FVector2f Icon16x16(16.f, 16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	Set("ClassIcon.SVGActor",                new IMAGE_BRUSH_SVG("Icons/DefaultSVG", Icon16x16));
	Set("ClassIcon.SVGShapesParentActor",    new IMAGE_BRUSH_SVG("Icons/DefaultSVG", Icon16x16));
	Set("ClassIcon.SVGShapeActor",           new IMAGE_BRUSH("Icons/irregularpolygon", Icon16x16));
	Set("ClassIcon.SVGDynamicMeshComponent", new IMAGE_BRUSH("Icons/irregularpolygon", Icon16x16));
	Set("SVGImporterEditor.SpawnSVGActor",   new IMAGE_BRUSH_SVG("Icons/DefaultSVG", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSVGImporterEditorStyle::~FSVGImporterEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
