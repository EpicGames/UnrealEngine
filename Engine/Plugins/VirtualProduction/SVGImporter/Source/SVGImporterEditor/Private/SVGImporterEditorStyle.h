// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FSVGImporterEditorStyle : public FSlateStyleSet
{
public:
	FSVGImporterEditorStyle();

	virtual ~FSVGImporterEditorStyle() override;

	static FSVGImporterEditorStyle& Get()
	{
		static FSVGImporterEditorStyle StyleSet;
		return StyleSet;
	}
};
