// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGImporterEditorUtils.h"
#include "SVGData.h"
#include "SVGDefines.h"
#include "SVGElements.h"
#include "SVGGraphicalElements.h"
#include "SVGImporterUtils.h"
#include "SVGParser_Base.h"
#include "SVGParsingUtils.h"
#include "Types/SVGRawAttribute.h"
#include "Types/SVGRawElement.h"

// SVGConstants are used throughout this file
using namespace UE::SVGImporter::Public;

TSharedPtr<FSVGDataInitializer> FSVGImporterEditorUtils::GetInitializerFromSVGData(const FString& InSVGTextBuffer, const FString& InSVGFileName /* == TEXT("") */)
{
	const TSharedRef<FSVGParser_Base> SVGParser = FSVGParsingUtils::CreateSVGParser(InSVGTextBuffer);

	const bool bParserSuccess = SVGParser->Parse();
	if (!bParserSuccess)
	{
		return nullptr;
	}

	const TSharedPtr<FSVGRawElement>& RootSVGElement = SVGParser->GetRootElement();
	if (!RootSVGElement)
	{
		return nullptr;
	}

	TSharedPtr<FSVGDataInitializer> SVGDataInitializer = MakeShared<FSVGDataInitializer>(InSVGTextBuffer, InSVGFileName);

	FSVGParsedElements ParsedElements;
	FSVGImporterEditorUtils::ParseRootRawElement(RootSVGElement.ToSharedRef(), ParsedElements);
	SVGDataInitializer->Elements = ParsedElements.GetOutElements();

	return SVGDataInitializer;
}

USVGData* FSVGImporterEditorUtils::CreateSVGDataFromTextBuffer(const FString& InSVGTextBuffer, UObject* InOuter, const FName InName, EObjectFlags InFlags, const FString& InSVGFileName /** == TEXT("") */)
{
	const TSharedPtr<FSVGDataInitializer>& SVGDataInitializer = GetInitializerFromSVGData(InSVGTextBuffer, InSVGFileName);

	if (!SVGDataInitializer)
	{
		return nullptr;
	}

	USVGData* SVGData = NewObject<USVGData>(InOuter, InName, InFlags);
	SVGData->Initialize(*SVGDataInitializer);

	return SVGData;
}

bool FSVGImporterEditorUtils::RefreshSVGDataFromTextBuffer(USVGData* InSVGData, const FString& InSVGTextBuffer)
{
	if (!InSVGData || InSVGTextBuffer.IsEmpty())
	{
		return false;
	}

	if (const TSharedPtr<FSVGDataInitializer>& SVGDataInitializer = GetInitializerFromSVGData(InSVGTextBuffer, InSVGData->GetSourceFilePath()))
	{
		InSVGData->Initialize(*SVGDataInitializer);

		return true;
	}

	return false;
}

void FSVGImporterEditorUtils::ParseRootRawElement(const TSharedRef<FSVGRawElement>& InRootElement, FSVGParsedElements& OutParsedElements)
{
	ParseSVGElement(InRootElement, OutParsedElements, nullptr);
}

TSharedRef<FSVGBaseElement> FSVGImporterEditorUtils::CreateSVGElement(const TSharedRef<FSVGRawElement>& InElement)
{
	const FSVGAttributesMap& AttributesMap = InElement->GetAttributes();

	if (IsValidSVG(InElement))
	{
		return CreateSVG();
	}
	else if (IsValidClipPath(InElement))
	{
		return CreateClipPath();
	}
	else if (IsValidStyle(InElement))
	{
		return CreateStyle(InElement->Value);
	}
	else if (IsValidLinearGradient(InElement) || IsValidRadialGradient(InElement))
	{
		// We are currently using the same function for both radial and linear gradients.
		// This could be further specialized with actual support for gradients in materials 
		return CreateGradient(AttributesMap, GetGradientStops(InElement));
	}
	else if (IsValidGroup(InElement))
	{
		return CreateGroup(AttributesMap);
	}
	else if (IsValidCircle(InElement))
	{
		return CreateCircle(AttributesMap);
	}
	else if (IsValidEllipse(InElement))
	{
		return CreateEllipse(AttributesMap);
	}
	else if (IsValidRectangle(InElement))
	{
		return CreateRectangle(AttributesMap);
	}
	else if (IsValidLine(InElement))
	{
		return CreateLine(AttributesMap);
	}
	else if (IsValidPolyLine(InElement))
	{
		return CreatePolyLine(AttributesMap);
	}
	else if (IsValidPolygon(InElement))
	{
		return CreatePolygon(AttributesMap);
	}
	else if (IsValidPath(InElement))
	{
		return CreatePath(AttributesMap);
	}

	return MakeShared<FSVGBaseElement>();
}

void FSVGImporterEditorUtils::SetGraphicsElementCommonValues(const TSharedPtr<FSVGGraphicsElement>& InGraphicalElement, const FSVGAttributesMap& InAttributesMap)
{
	if (!InGraphicalElement)
	{
		return;
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeTransform = GetAttribute(SVGConstants::Transform, InAttributesMap))
	{
		InGraphicalElement->SetTransform(AttributeTransform->AsString());
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeID = GetAttribute(SVGConstants::Id, InAttributesMap))
	{
		InGraphicalElement->SetName(AttributeID->AsString());
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeStyleClassName = GetAttribute(SVGConstants::Class, InAttributesMap))
	{
		InGraphicalElement->SetStyleClassName(AttributeStyleClassName->AsString());
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeStroke = GetAttribute(SVGConstants::Stroke, InAttributesMap))
	{
		const FString& StrokeString = AttributeStroke->AsString();

		if (FSVGParsingUtils::IsURLString(StrokeString))
		{
			InGraphicalElement->SetStrokeGradientID(FSVGParsingUtils::ExtractURL(StrokeString));
		}
		else
		{
			InGraphicalElement->SetStrokeColor(AttributeStroke->AsString());
		}

		InGraphicalElement->SetStrokeColor(AttributeStroke->AsString());
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeStrokeWidth = GetAttribute(SVGConstants::StrokeWidth, InAttributesMap))
	{
		InGraphicalElement->SetStrokeWidth(AttributeStrokeWidth->AsFloat());
	}

	if (const TSharedPtr<FSVGRawAttribute> AttributeFill = GetAttribute(SVGConstants::Fill, InAttributesMap))
	{
		const FString& FillString = AttributeFill->AsString();

		if (FSVGParsingUtils::IsURLString(FillString))
		{
			InGraphicalElement->SetFillGradientID(FSVGParsingUtils::ExtractURL(FillString));
		}
		else
		{
			InGraphicalElement->SetFillColor(AttributeFill->AsString());
		}
	}
}

void FSVGImporterEditorUtils::ParseSVGElement(const TSharedRef<FSVGRawElement>& InElement, FSVGParsedElements& OutParsedElements, TSharedPtr<FSVGBaseElement> InParent)
{
	const TSharedRef<FSVGBaseElement> NewElement = CreateSVGElement(InElement);

	if (NewElement->TypeIsSet())
	{
		if (NewElement->IsGraphicElement())
		{
			const TSharedRef<FSVGGraphicsElement> NewElementAsGraphicElem = StaticCastSharedRef<FSVGGraphicsElement>(NewElement);

			NewElementAsGraphicElem->TryRegisterWithParent(InParent);

			const FSVGAttributesMap& AttributesMap = InElement->GetAttributes();

			SetGraphicsElementCommonValues(NewElementAsGraphicElem, AttributesMap);

			bool bStyleHasBeenSetup = false;

			if (const TSharedPtr<FSVGRawAttribute> AttributeStyle = GetAttribute(SVGConstants::Style, AttributesMap))
			{
				// todo: handle this case in the style parser itself, so we don't have to add brackets here
				FString StyleString = TEXT("{") + AttributeStyle->AsString().TrimStartAndEnd() + TEXT("}");

				TArray<FSVGStyle> Styles = FSVGImporterUtils::StylesFromCSS(MoveTemp(StyleString));

				// Really, this should be just one
				if (Styles.Num() > 0) 
				{
					NewElementAsGraphicElem->SetStyle(Styles[0]);
					bStyleHasBeenSetup = true;
				}
			}

			// If no class or style was found, check parent
			if (!bStyleHasBeenSetup && InParent)
			{
				// In case the element which triggered the creation of this one can act as a hierarchical parent (e.g. "g")
				if (InParent->Type == ESVGElementType::Group)
				{
					if (const TSharedPtr<FSVGGraphicsElement> ParentAsGraphicElem = StaticCastSharedPtr<FSVGGraphicsElement>(InParent))
					{
						if (!AttributesMap.Contains(SVGConstants::Stroke) && ParentAsGraphicElem->HasStroke())
						{
							NewElementAsGraphicElem->SetStrokeColor(ParentAsGraphicElem->GetStrokeColor());
						}

						if (!AttributesMap.Contains(SVGConstants::Fill) && ParentAsGraphicElem->HasFill())
						{
							NewElementAsGraphicElem->SetFillColor(ParentAsGraphicElem->GetFillColor());
						}
					}
				}
			}
		}
		else if (NewElement->Type == ESVGElementType::Style)
		{
			const TSharedRef<FSVGStyleElement> StyleElem = StaticCastSharedRef<FSVGStyleElement>(NewElement);
			OutParsedElements.AddStyleElement(StyleElem);
		}
		else if (NewElement->Type == ESVGElementType::Gradient)
		{
			const TSharedRef<FSVGGradientElement> GradientElement = StaticCastSharedRef<FSVGGradientElement>(NewElement);
			OutParsedElements.AddGradientElement(GradientElement);
		}
		// todo: other cases
		// else {}

		// todo: clip paths are currently not handled.
		// We do need to take them into account though, otherwise they might be drawn as regular paths
		// In case this element has a clip path as parent, we currently skip it and don't add it to OutElements
		bool bParentIsClipPath = false;
		if (InParent)
		{
			bParentIsClipPath = InParent->Type == ESVGElementType::ClipPath;
		}

		if (!bParentIsClipPath)
		{
			OutParsedElements.AddElement(NewElement);
		}
	}

	TSharedPtr<FSVGBaseElement>& NextParent = InParent;

	if (NewElement->Type == ESVGElementType::Group || NewElement->Type == ESVGElementType::ClipPath)
	{
		NextParent = NewElement;
	}

	for (const TSharedPtr<FSVGRawElement>& Element : InElement->GetChildren())
	{
		if (Element)
		{
			ParseSVGElement(Element.ToSharedRef(), OutParsedElements, NextParent);
		}
	}
}

bool FSVGImporterEditorUtils::IsValidGroup(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == UE::SVGImporter::Public::SVGConstants::G;
}

bool FSVGImporterEditorUtils::IsValidSVG(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == UE::SVGImporter::Public::SVGConstants::SVG;
}

bool FSVGImporterEditorUtils::IsValidStyle(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == UE::SVGImporter::Public::SVGConstants::Style;
}

bool FSVGImporterEditorUtils::IsValidClipPath(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == UE::SVGImporter::Public::SVGConstants::ClipPath;
}

bool FSVGImporterEditorUtils::IsValidLinearGradient(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::LinearGradient;
}

bool FSVGImporterEditorUtils::IsValidRadialGradient(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::RadialGradient;
}

bool FSVGImporterEditorUtils::IsValidCircle(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Circle;
}

bool FSVGImporterEditorUtils::IsValidEllipse(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Ellipse;
}

bool FSVGImporterEditorUtils::IsValidRectangle(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Rect;
}

bool FSVGImporterEditorUtils::IsValidLine(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Line &&
		InElement->HasAttribute(SVGConstants::X1) &&
		InElement->HasAttribute(SVGConstants::Y1) &&
		InElement->HasAttribute(SVGConstants::X2) &&
		InElement->HasAttribute(SVGConstants::Y2);
}

bool FSVGImporterEditorUtils::IsValidPolyLine(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::PolyLine &&
		InElement->HasAttribute(SVGConstants::Points);
}

bool FSVGImporterEditorUtils::IsValidPolygon(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Polygon &&
		InElement->HasAttribute(SVGConstants::Points);
}

bool FSVGImporterEditorUtils::IsValidPath(const TSharedRef<FSVGRawElement>& InElement)
{
	return InElement->Name == SVGConstants::Path
		&& InElement->HasAttribute(SVGConstants::D);
}

TSharedRef<FSVGMainElement> FSVGImporterEditorUtils::CreateSVG()
{
	return MakeShared<FSVGMainElement>();
}

TSharedRef<FSVGGroupElement> FSVGImporterEditorUtils::CreateGroup(const FSVGAttributesMap& InAttributesMap)
{
	TSharedRef<FSVGGroupElement> GroupElement = MakeShared<FSVGGroupElement>();

	//@note: children add themselves to their parent group

	if (InAttributesMap.Contains(SVGConstants::Stroke))
	{
		if (const TSharedPtr<FSVGRawAttribute> AttributeStroke = GetAttribute(SVGConstants::Stroke, InAttributesMap))
		{
			GroupElement->SetStrokeColor(AttributeStroke->AsString());
		}
	}

	return GroupElement;
}

TSharedRef<FSVGClipPath> FSVGImporterEditorUtils::CreateClipPath()
{
	return MakeShared<FSVGClipPath>();
}

TSharedRef<FSVGStyleElement> FSVGImporterEditorUtils::CreateStyle(const FString& InCSS_String)
{
	const TArray<FSVGStyle>& Styles = FSVGImporterUtils::StylesFromCSS(InCSS_String);
	return MakeShared<FSVGStyleElement>(Styles);
}

TArray<FSVGGradientStop> FSVGImporterEditorUtils::GetGradientStops(const TSharedRef<FSVGRawElement>& InElement)
{
	TArray<FSVGGradientStop> GradientStops;

	for (const TSharedPtr<FSVGRawElement>& Child : InElement->GetChildren())
	{
		if (Child->Name == SVGConstants::GradientStop)
		{
			const TMap<FName, TSharedPtr<FSVGRawAttribute>>& AttributesMap = Child->GetAttributes();

			const TSharedPtr<FSVGRawAttribute> OffsetAttribute = GetAttribute(SVGConstants::Offset, AttributesMap);
			const TSharedPtr<FSVGRawAttribute> StopColorAttribute = GetAttribute(SVGConstants::StopColor, AttributesMap);
			const TSharedPtr<FSVGRawAttribute> StopOpacityAttribute = GetAttribute(SVGConstants::StopOpacity, AttributesMap);

			float Offset = 0.0f;
			FColor StopColor = FColor::Black;
			float StopOpacity = 1.0f;

			if (OffsetAttribute.IsValid())
			{
				Offset = OffsetAttribute->AsFloat();
			}

			if (StopColorAttribute.IsValid())
			{
				StopColor = FSVGImporterUtils::GetColorFromSVGString(StopColorAttribute->AsString());
			}

			if (StopOpacityAttribute.IsValid())
			{
				StopOpacity = StopOpacityAttribute->AsFloat();
			}

			FSVGGradientStop GradientStop;
			GradientStop.Offset = Offset;
			GradientStop.StopColor = StopColor;
			GradientStop.StopOpacity = StopOpacity;

			GradientStops.Add(GradientStop);

			GradientStops.Append(GetGradientStops(Child.ToSharedRef()));
		}
		else
		{
			break;
		}
	}

	return GradientStops;
}

TSharedRef<FSVGGradientElement> FSVGImporterEditorUtils::CreateGradient(const FSVGAttributesMap& InAttributesMap, const TArray<FSVGGradientStop>& InGradientStops)
{
	const TSharedPtr<FSVGRawAttribute> AttributeX1 = GetAttribute(SVGConstants::X1, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeY1 = GetAttribute(SVGConstants::Y1, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeX2 = GetAttribute(SVGConstants::X2, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeY2 = GetAttribute(SVGConstants::Y2, InAttributesMap);

	FString GradientUnitsStr;
	if (const TSharedPtr<FSVGRawAttribute> AttributeGradientUnits = GetAttribute(SVGConstants::GradientUnits, InAttributesMap))
	{
		GradientUnitsStr = AttributeGradientUnits->AsString();
	}

	FString Id;
	if (const TSharedPtr<FSVGRawAttribute> AttributeId = GetAttribute(SVGConstants::Id, InAttributesMap))
	{
		Id = AttributeId->AsString();
	}

	ESVGGradientPointUnit GradientUnit = ESVGGradientPointUnit::ObjectBoundingBox;
	if (!GradientUnitsStr.IsEmpty())
	{
		if (GradientUnitsStr == SVGConstants::UserSpaceOnUse)
		{
			GradientUnit = ESVGGradientPointUnit::ObjectBoundingBox;
		}
		else if (GradientUnitsStr == SVGConstants::ObjectBoundingBox)
		{
			GradientUnit = ESVGGradientPointUnit::UserSpaceOnUse;
		}
	}

	float X1 = 0.0f;
	float Y1 = 0.0f;
	float X2 = 0.0f;
	float Y2 = 0.0f;

	if (GradientUnit == ESVGGradientPointUnit::ObjectBoundingBox)
	{
		if (AttributeX1)
		{
			AttributeX1->AsFloatWithSuffix(SVGConstants::Percentage, X1);
		}

		if (AttributeY1)
		{
			AttributeY1->AsFloatWithSuffix(SVGConstants::Percentage, Y1);
		}

		if (AttributeX2)
		{
			AttributeX2->AsFloatWithSuffix(SVGConstants::Percentage, X2);
		}

		if (AttributeY2)
		{
			AttributeY2->AsFloatWithSuffix(SVGConstants::Percentage, Y2);
		}
	}
	else
	{
		if (AttributeX1)
		{
			X1 = AttributeX1->AsFloat();
		}

		if (AttributeY1)
		{
			Y1 = AttributeY1->AsFloat();
		}

		if (AttributeX2)
		{
			X2 = AttributeX2->AsFloat();
		}

		if (AttributeY2)
		{
			Y2 = AttributeY2->AsFloat();
		}
	}

	FSVGGradient LinearGradient(Id, GradientUnit, FVector2D(X1, Y1), FVector2D(X2, Y2), InGradientStops);

	return MakeShared<FSVGGradientElement>(LinearGradient);
}

TSharedRef<FSVGCircle> FSVGImporterEditorUtils::CreateCircle(const FSVGAttributesMap& InAttributesMap)
{
	float Cx;
	float Cy;
	GetCxAndCy(InAttributesMap, Cx, Cy);

	const float R = GetR(InAttributesMap);

	return MakeShared<FSVGCircle>(Cx, Cy, R);
}

TSharedRef<FSVGEllipse> FSVGImporterEditorUtils::CreateEllipse(const FSVGAttributesMap& InAttributesMap)
{
	float Cx;
	float Cy;
	GetCxAndCy(InAttributesMap, Cx, Cy);

	float Rx;
	float Ry;
	GetRxAndRy(InAttributesMap, Rx, Ry);

	return MakeShared<FSVGEllipse>(Cx, Cy, Rx, Ry);
}

TSharedRef<FSVGRectangle> FSVGImporterEditorUtils::CreateRectangle(const FSVGAttributesMap& InAttributesMap)
{
	float X;
	float Y;
	GetXAndY(InAttributesMap, X, Y);

	float Width;
	float Height;
	GetWidthAndHeight(InAttributesMap, Width, Height);

	float Rx;
	float Ry;
	GetRxAndRy(InAttributesMap, Rx, Ry);

	return MakeShared<FSVGRectangle>(X, Y, Width, Height, Rx, Ry);
}

TSharedRef<FSVGLine> FSVGImporterEditorUtils::CreateLine(const FSVGAttributesMap& InAttributesMap)
{
	const TSharedPtr<FSVGRawAttribute> AttributeX1 = GetAttribute(SVGConstants::X1, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeY1 = GetAttribute(SVGConstants::Y1, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeX2 = GetAttribute(SVGConstants::X2, InAttributesMap);
	const TSharedPtr<FSVGRawAttribute> AttributeY2 = GetAttribute(SVGConstants::Y2, InAttributesMap);

	const float X1 = AttributeX1->AsFloat();
	const float Y1 = AttributeY1->AsFloat();
	const float X2 = AttributeX2->AsFloat();
	const float Y2 = AttributeY2->AsFloat();

	float PathLength = 0.0f;
	if (const TSharedPtr<FSVGRawAttribute> AttributePathLength = GetAttribute(SVGConstants::PathLength, InAttributesMap))
	{
		PathLength = AttributePathLength->AsFloat();
	}

	return MakeShared<FSVGLine>(PathLength, X1, Y1, X2, Y2);
}

TSharedRef<FSVGPolyLine> FSVGImporterEditorUtils::CreatePolyLine(const FSVGAttributesMap& InAttributesMap)
{
	TArray<FVector2D> Points;
	if (const TSharedPtr<FSVGRawAttribute> AttributePoints = GetAttribute(SVGConstants::Points, InAttributesMap))
	{
		AttributePoints->AsPoints(Points);
	}

	float PathLength = 0.0f;

	if (const TSharedPtr<FSVGRawAttribute> AttributePathLength = GetAttribute(SVGConstants::PathLength, InAttributesMap))
	{
		PathLength = AttributePathLength->AsFloat();
	}

	return MakeShared<FSVGPolyLine>(PathLength, Points);
}

TSharedRef<FSVGPolygon> FSVGImporterEditorUtils::CreatePolygon(const FSVGAttributesMap& InAttributesMap)
{
	TArray<FVector2D> Points;
	if (const TSharedPtr<FSVGRawAttribute> AttributePoints = GetAttribute(SVGConstants::Points, InAttributesMap))
	{
		AttributePoints->AsPoints(Points);
	}

	float PathLength = 0.0;

	if (const TSharedPtr<FSVGRawAttribute> AttributePathLength = GetAttribute(SVGConstants::PathLength, InAttributesMap))
	{
		PathLength = AttributePathLength->AsFloat();
	}

	return MakeShared<FSVGPolygon>(PathLength, Points);
}

TSharedRef<FSVGPath> FSVGImporterEditorUtils::CreatePath(const FSVGAttributesMap& InAttributesMap)
{
	float PathLength = 0.0f;

	if (const TSharedPtr<FSVGRawAttribute> AttributePathLength = GetAttribute(SVGConstants::PathLength, InAttributesMap))
	{
		PathLength = AttributePathLength->AsFloat();
	}

	TArray<TArray<FSVGPathCommand>> Paths;
	if (const TSharedPtr<FSVGRawAttribute> Attributed = GetAttribute(SVGConstants::D, InAttributesMap))
	{
		Paths = Attributed->AsPathCommands();
	}

	return MakeShared<FSVGPath>(Paths, PathLength);
}

bool FSVGImporterEditorUtils::HasAttribute(const FName InAttributeName, const FSVGAttributesMap& InAttributes)
{
	return InAttributes.Contains(InAttributeName);
}

TSharedPtr<FSVGRawAttribute> FSVGImporterEditorUtils::GetAttribute(const FName InAttributeName, const FSVGAttributesMap& InAttributes)
{
	if (const TSharedPtr<FSVGRawAttribute>* Attribute = InAttributes.Find(InAttributeName))
	{
		return *Attribute;
	}

	return nullptr;
}

void FSVGImporterEditorUtils::GetRxAndRy(const FSVGAttributesMap& InAttributesMap, float& OutRx, float& OutRy)
{
	OutRx = 0.0f;
	OutRy = 0.0f;

	if (const TSharedPtr<FSVGRawAttribute> Rx = GetAttribute(SVGConstants::RX, InAttributesMap))
	{
		OutRx = Rx->AsFloat();
	}

	if (const TSharedPtr<FSVGRawAttribute> Ry = GetAttribute(SVGConstants::RY, InAttributesMap))
	{
		OutRy = Ry->AsFloat();
	}
}

void FSVGImporterEditorUtils::GetCxAndCy(const FSVGAttributesMap& InAttributesMap, float& OutCx, float& OutCy)
{
	OutCx = GetCx(InAttributesMap);
	OutCy = GetCy(InAttributesMap);
}

void FSVGImporterEditorUtils::GetXAndY(const FSVGAttributesMap& InAttributesMap, float& OutX, float& OutY)
{
	OutX = GetX(InAttributesMap);
	OutY = GetY(InAttributesMap);
}

void FSVGImporterEditorUtils::GetWidthAndHeight(const FSVGAttributesMap& InAttributesMap, float& OutWidth, float& OutHeight)
{
	OutWidth = GetWidth(InAttributesMap);
	OutHeight = GetHeight(InAttributesMap);
}

float FSVGImporterEditorUtils::GetCx(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> Cx = GetAttribute(SVGConstants::CX, InAttributesMap))
	{
		return Cx->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetCy(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> Cy = GetAttribute(SVGConstants::CY, InAttributesMap))
	{
		return Cy->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetR(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> R = GetAttribute(SVGConstants::R, InAttributesMap))
	{
		return R->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetX(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> X = GetAttribute(SVGConstants::X, InAttributesMap))
	{
		return X->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetY(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> Y = GetAttribute(SVGConstants::Y, InAttributesMap))
	{
		return Y->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetWidth(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> Width = GetAttribute(SVGConstants::Width, InAttributesMap))
	{
		return Width->AsFloat();
	}

	return 0.0f;
}

float FSVGImporterEditorUtils::GetHeight(const FSVGAttributesMap& InAttributesMap)
{
	if (const TSharedPtr<FSVGRawAttribute> Height = GetAttribute(SVGConstants::Height, InAttributesMap))
	{
		return Height->AsFloat();
	}

	return 0.0f;
}

bool FSVGImporterEditorUtils::FSVGParsedElements::HasStyleClass(const FString& InClassName) const
{
	return StyledGraphicsElementsMap.Contains(InClassName);
}

bool FSVGImporterEditorUtils::FSVGParsedElements::HasGradient(const FString& InGradientID) const
{
	return GradientGraphicsElementsMap.Contains(InGradientID);
}

const TArray<TSharedRef<FSVGBaseElement>>& FSVGImporterEditorUtils::FSVGParsedElements::GetOutElements() const
{
	if (!bStyleHaveBeenApplied || !bGradientsHaveBeenApplied)
	{
		FSVGParsedElements* This = const_cast<FSVGParsedElements*>(this);
		This->ApplyPendingData();
	}

	return OutElements;
}

void FSVGImporterEditorUtils::FSVGParsedElements::MarkIfMissingStyle(const TSharedRef<FSVGGraphicsElement>& InGraphicElement)
{
	const FString& StyleClassName = InGraphicElement->GetStyleClassName();

	if (StyleClassName.IsEmpty())
	{
		return;
	}

	if (!HasStyleClass(StyleClassName))
	{
		StyledGraphicsElementsMap.Add(StyleClassName);
	}

	// If this element has a class attribute, let's mark it for applying that class style later
	// We cannot apply the style in place, because sometimes styles are defined after the elements using them
	StyledGraphicsElementsMap[StyleClassName].AddUnique(InGraphicElement);
	bStyleHaveBeenApplied = false;
}

void FSVGImporterEditorUtils::FSVGParsedElements::MarkIfMissingGradients(const TSharedRef<FSVGGraphicsElement>& InGraphicElement)
{
	if (InGraphicElement->HasFillGradientID())
	{
		const FString& FillGradientId = InGraphicElement->GetFillGradientID();

		if (!HasGradient(FillGradientId))
		{
			GradientGraphicsElementsMap.Add(FillGradientId);
		}

		GradientGraphicsElementsMap[FillGradientId].AddUnique(InGraphicElement);
		bGradientsHaveBeenApplied = false;
	}

	if (InGraphicElement->HasStrokeGradientID())
	{
		const FString& StrokeGradientId = InGraphicElement->GetStrokeGradientID();

		if (!HasGradient(StrokeGradientId))
		{
			GradientGraphicsElementsMap.Add(StrokeGradientId);
		}

		GradientGraphicsElementsMap[StrokeGradientId].AddUnique(InGraphicElement);
		bGradientsHaveBeenApplied = false;
	}
}

void FSVGImporterEditorUtils::FSVGParsedElements::AddElement(const TSharedRef<FSVGBaseElement>& InElement)
{
	OutElements.Add(InElement);

	if (InElement->IsGraphicElement())
	{
		const TSharedRef<FSVGGraphicsElement> AsGraphicElement = StaticCastSharedRef<FSVGGraphicsElement>(InElement);

		MarkIfMissingStyle(AsGraphicElement);
		MarkIfMissingGradients(AsGraphicElement);
	}
}

void FSVGImporterEditorUtils::FSVGParsedElements::AddStyleElement(const TSharedRef<FSVGStyleElement>& InStyle)
{
	StyleElements.Add(InStyle);
	bStyleHaveBeenApplied = false;
}

void FSVGImporterEditorUtils::FSVGParsedElements::AddGradientElement(const TSharedRef<FSVGGradientElement>& InGradientElement)
{
	GradientElements.Add(InGradientElement);
	bGradientsHaveBeenApplied = false;
}

void FSVGImporterEditorUtils::FSVGParsedElements::ApplyStyles()
{
	if (bStyleHaveBeenApplied)
	{
		return;
	}

	// Applying styles after parsing, in order to address the possibility of styles being defined after elements using them (seen in some SVGs)
	bStyleHaveBeenApplied = true;

	if (StyleElements.IsEmpty())
	{
		return;
	}

	for (const TPair<FString, TArray<TSharedRef<FSVGGraphicsElement>>>& StyledGraphicsElementsPair : StyledGraphicsElementsMap)
	{
		for (const TSharedRef<FSVGGraphicsElement>& GraphicsElement : StyledGraphicsElementsPair.Value)
		{
			for (const TSharedRef<FSVGStyleElement>& StyleElem : StyleElements)
			{
				FSVGStyle Style;

				if (StyleElem->GetStyleByClassName(GraphicsElement->GetStyleClassName(), Style))
				{
					GraphicsElement->SetStyle(Style);
				}
			}
		}
	}
}

void FSVGImporterEditorUtils::FSVGParsedElements::ApplyGradients()
{
	if (bGradientsHaveBeenApplied)
	{
		return;
	}

	// Applying styles after parsing, in order to address the possibility of styles being defined after elements using them (seen in some SVGs)
	bGradientsHaveBeenApplied = true;

	if (GradientElements.IsEmpty())
	{
		return;
	}

	for (const TPair<FString, TArray<TSharedRef<FSVGGraphicsElement>>>& GradientGraphicsElementsPair : GradientGraphicsElementsMap)
	{
		for (const TSharedRef<FSVGGraphicsElement>& GraphicsElement : GradientGraphicsElementsPair.Value)
		{
			for (const TSharedRef<FSVGGradientElement>& GradientElement : GradientElements)
			{
				FSVGGradient Gradient;

				if (GradientElement->HasID())
				{
					if (GraphicsElement->GetFillGradientID() == GradientElement->GetID())
					{
						GraphicsElement->SetFillGradient(GradientElement->Gradient);
					}

					if (GraphicsElement->GetStrokeGradientID() == GradientElement->GetID())
					{
						GraphicsElement->SetStrokeGradient(GradientElement->Gradient);
					}
				}
			}
		}
	}
}

void FSVGImporterEditorUtils::FSVGParsedElements::MarkHiddenElements()
{
	for (const TSharedRef<FSVGBaseElement>& Element : OutElements)
	{
		if (Element->Type == ESVGElementType::Group)
		{
			const TSharedRef<FSVGGroupElement> GroupElement = StaticCastSharedRef<FSVGGroupElement>(Element);

			if (!GroupElement->IsVisible())
			{
				for (const TSharedRef<FSVGGraphicsElement>& ChildElement : GroupElement->Children)
				{
					ChildElement->Hide();
				}
			}
		}
	}
}

void FSVGImporterEditorUtils::FSVGParsedElements::ApplyPendingData()
{
	ApplyStyles();
	ApplyGradients();
	MarkHiddenElements();
}
