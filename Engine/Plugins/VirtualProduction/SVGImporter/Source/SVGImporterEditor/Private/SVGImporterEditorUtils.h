// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGData.h"

class USVGData;
struct FSVGBaseElement;
struct FSVGCircle;
struct FSVGClipPath;
struct FSVGEllipse;
struct FSVGGradientElement;
struct FSVGGraphicsElement;
struct FSVGGroupElement;
struct FSVGLine;
struct FSVGMainElement;
struct FSVGPath;
struct FSVGPolyLine;
struct FSVGPolygon;
struct FSVGRawAttribute;
struct FSVGRawElement;
struct FSVGRectangle;
struct FSVGStyleElement;

typedef TMap<FName, TSharedPtr<FSVGRawAttribute>> FSVGAttributesMap;

class FSVGImporterEditorUtils
{
	/**
	 * Struct used to hold and store data during XML --> SVG parsing
	 */
	struct FSVGParsedElements : public TSharedFromThis<FSVGParsedElements>
	{
		/** Get all parsed SVG elements. Will also call ApplyStyle() if needed, so that CSS style classes are applied to elements requiring it */
		const TArray<TSharedRef<FSVGBaseElement>>& GetOutElements() const;

		/** Register a parsed SVG Element. If needed, style class info can be applied at the end of parsing by calling ApplyStyles() */
		void AddElement(const TSharedRef<FSVGBaseElement>& InElement);

		/** Register a Style Element to be applied later via ApplyStyles() */
		void AddStyleElement(const TSharedRef<FSVGStyleElement>& InStyle);

		/** Register a Gradient Element to be applied later */
		void AddGradientElement(const TSharedRef<FSVGGradientElement>& InGradientElement);

	protected:
		/** Apply Style Elements to Parsed Graphics Elements, as needed */
		void ApplyStyles();

		/** Apply gradients, as needed */
		void ApplyGradients();

		/** Marks shapes as non visible, based on the "Display" attribute */
		void MarkHiddenElements();

		/** Applies all pending data (e.g. styles, gradients, visibility) */
		void ApplyPendingData();

		/** Checks whether the specified Class is available */
		bool HasStyleClass(const FString& InClassName) const;

		/** Checks whether the specified Gradient is available */
		bool HasGradient(const FString& InGradientID) const;

		/** If the specified Element needs a style class which is not available yet, let's register it for later update */
		void MarkIfMissingStyle(const TSharedRef<FSVGGraphicsElement>& InGraphicElement);

		/** If the specified Element needs a gradient which is not available yet, let's register it for later update */
		void MarkIfMissingGradients(const TSharedRef<FSVGGraphicsElement>& InGraphicElement);

		/** Style Elements containing style classes which should be applied to StyledGraphicsElementsMap once parsing is finished*/
		TArray<TSharedRef<FSVGStyleElement>> StyleElements;

		/** Gradient Elements containing gradients to be applied once parsing is finished*/
		TArray<TSharedRef<FSVGGradientElement>> GradientElements;

		/** The elements needed to generate the 3D geometry for this SVG Data*/
		TArray<TSharedRef<FSVGBaseElement>> OutElements;

		/** Graphics elements needing style from style classes */
		TMap<FString, TArray<TSharedRef<FSVGGraphicsElement>>> StyledGraphicsElementsMap;

		/** Graphics elements needing gradient info */
		TMap<FString, TArray<TSharedRef<FSVGGraphicsElement>>> GradientGraphicsElementsMap;

		/** Set to true after calling ApplyStyle, set to false when Graphic Elements or StyleElements are updated. */
		bool bStyleHaveBeenApplied = false;

		/** Set to true after calling ApplyGradients, set to false when Graphic Elements or GradientElements are updated. */
		bool bGradientsHaveBeenApplied = false;
	};

public:
	/**
	 * Creates a new SVGData from the specified Text Buffer
	 * @param InSVGTextBuffer Text buffer containing the SVG text
	 * @param InOuter Outer for the newly created SVGData
	 * @param InName Name for the SVGData Object
	 * @param InFlags Desired Object Flags
	 * @param InSVGFileName If the SVG Text buffer is taken from a file, it is possible specify here its name
	 * @return the newly created SVGData. Can be nullptr if parsing of SVG text buffer fails.
	 */
	static USVGData* CreateSVGDataFromTextBuffer(const FString& InSVGTextBuffer, UObject* InOuter, const FName InName, EObjectFlags InFlags, const FString& InSVGFileName = TEXT(""));

	/**
	 * Refresh an existing SVGData using the specified Text Buffer
	 * @param InSVGData The SVGData object to update
	 * @param InSVGTextBuffer Text buffer containing the SVG text
	 * @return the newly created SVGData. Can be nullptr if parsing of SVG text buffer fails.
	 */
	static bool RefreshSVGDataFromTextBuffer(USVGData* InSVGData, const FString& InSVGTextBuffer);

	/**
	 * Creates a SVGDataInitializer struct which can be used to initialize a new SVGData, or refresh a new one
	 * @param InSVGTextBuffer Text buffer containing the SVG text
	 * @param InSVGFileName If the SVG Text buffer is taken from a file, it is possible specify here its name
	 * @return the FSVGDataInitializer. It will contain the hierarchy of SVG Elements needed to properly setup SVGData objects,
	 * the content of the SVG, and the filename, when available.
	 */
	static TSharedPtr<FSVGDataInitializer> GetInitializerFromSVGData(const FString& InSVGTextBuffer, const FString& InSVGFileName = TEXT(""));

	/**
	 * Converts the hierarchy starting from the specified SVG Raw Element to a list of more specialized SVG Base Elements which can be used to generate geometry
	 * @param InRootElement the SVG root element obtained after parsing an SVG Text Buffer
	 * @param OutParsedElements the output struct of SVG Parsed Elements. New Elements and required data (e.g. style information) will be added to this list.
	 */
	static void ParseRootRawElement(const TSharedRef<FSVGRawElement>& InRootElement, FSVGParsedElements& OutParsedElements);

private:
	/**
	 * Converts the hierarchy starting from the specified SVG Raw Element to a list of more specialized SVG Base Elements which can be used to generate geometry
	 * @param InElement the SVG element to parse
	 * @param OutParsedElements the output struct of SVG Parsed Elements. New Elements and required data (e.g. style information) will be added to this list.
	 * @param InParent the parent of the specified Element. Can be null (e.g. when parsing root elements)
	 */
	static void ParseSVGElement(const TSharedRef<FSVGRawElement>& InElement, FSVGParsedElements& OutParsedElements, TSharedPtr<FSVGBaseElement> InParent);

	/**
	 * Will create a specialized SVG Base Element. Specialization will be based on the current arguments of the specified Raw Element.
	 * @param InElement the SVG element from which the specialized Element will be created
	 * @return the newly created Base Element
	 */
	static TSharedRef<FSVGBaseElement> CreateSVGElement(const TSharedRef<FSVGRawElement>& InElement);

	/**
	 * Graphical SVG Elements share a common set of attributes.
	 * This function will set those attributes, where possible, using the specified Attributes Map
	 * @param InGraphicalElement the element to update
	 * @param InAttributesMap the attributes used to update the Graphical Element
	 */
	static void SetGraphicsElementCommonValues(const TSharedPtr<FSVGGraphicsElement>& InGraphicalElement, const FSVGAttributesMap& InAttributesMap);

	/** Is this Raw Element a Group? */
	static bool IsValidGroup(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element an SVG? */
	static bool IsValidSVG(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Style? */
	static bool IsValidStyle(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Clip Path? */
	static bool IsValidClipPath(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Linear Gradient? */
	static bool IsValidLinearGradient(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Radial Gradient? */
	static bool IsValidRadialGradient(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Circle? */
	static bool IsValidCircle(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element an Ellipse? */
	static bool IsValidEllipse(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Rectangle? */
	static bool IsValidRectangle(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Line? */
	static bool IsValidLine(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Poly Line? */
	static bool IsValidPolyLine(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Polygon? */
	static bool IsValidPolygon(const TSharedRef<FSVGRawElement>& InElement);

	/** Is this Raw Element a Path? */
	static bool IsValidPath(const TSharedRef<FSVGRawElement>& InElement);

	/** Creates an SVG Element*/
	static TSharedRef<FSVGMainElement> CreateSVG();

	/** Creates a Group Element using the specified Attributes and Outer */
	static TSharedRef<FSVGGroupElement> CreateGroup(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Path Element */
	static TSharedRef<FSVGClipPath> CreateClipPath();

	/** Creates a Style Element using the specified CSS String and Outer*/
	static TSharedRef<FSVGStyleElement> CreateStyle(const FString& InCSS_String);

	/** Retrieve the list of gradient stops from the specified Elements, if any */
	static TArray<FSVGGradientStop> GetGradientStops(const TSharedRef<FSVGRawElement>& InElement);

	/** Creates a Gradient Element */
	static TSharedRef<FSVGGradientElement> CreateGradient(const FSVGAttributesMap& InAttributesMap, const TArray<FSVGGradientStop>& InGradientStops);

	/** Creates a Circle Element using the specified Attributes and Outer */
	static TSharedRef<FSVGCircle> CreateCircle(const FSVGAttributesMap& InAttributesMap);

	/** Creates an Ellipse Element using the specified Attributes and Outer */
	static TSharedRef<FSVGEllipse> CreateEllipse(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Rectangle Element using the specified Attributes and Outer */
	static TSharedRef<FSVGRectangle> CreateRectangle(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Line Element using the specified Attributes and Outer */
	static TSharedRef<FSVGLine> CreateLine(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Poly Line Elemen using the specified Attributes and Outer */
	static TSharedRef<FSVGPolyLine> CreatePolyLine(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Polygon Element using the specified Attributes and Outer */
	static TSharedRef<FSVGPolygon> CreatePolygon(const FSVGAttributesMap& InAttributesMap);

	/** Creates a Path Element using the specified Attributes and Outer */
	static TSharedRef<FSVGPath> CreatePath(const FSVGAttributesMap& InAttributesMap);

	/** Checks if the specified Attributes map contains an Attribute */
	static bool HasAttribute(const FName InAttributeName, const FSVGAttributesMap& InAttributes);

	/** Returns the specified Attribute from the specified Attributes Map */
	static TSharedPtr<FSVGRawAttribute> GetAttribute(const FName InAttributeName, const FSVGAttributesMap& InAttributes);

	/**
	 * Get Horizontal and Vertical Radius Values from the specified AttributesMap, if available.
	 * 
	 * Rx and Ry default value is auto
	 * When Rx is auto, the used radius is equal to the absolute length used for Ry (generates a circular arc)
	 * If both Rx and Ry are auto, used value is 0
	 */
	static void GetRxAndRy(const FSVGAttributesMap& InAttributesMap, float& OutRx, float& OutRy);

	/** Get both Cx and Cy Coordinates from the specified AttributesMap, if available */
	static void GetCxAndCy(const FSVGAttributesMap& InAttributesMap, float& OutCx, float& OutCy);

	/** Get both X and Y Coordinates from the specified AttributesMap, if available */
	static void GetXAndY(const FSVGAttributesMap& InAttributesMap, float& OutX, float& OutY);

	/** Get both Width and Height Coordinates from the specified AttributesMap, if available */
	static void GetWidthAndHeight(const FSVGAttributesMap& InAttributesMap, float& OutWidth, float& OutHeight);

	/** Get Horizontal Center Value from the specified AttributesMap, if available */
	static float GetCx(const FSVGAttributesMap& InAttributesMap);

	/** Get Vertical Center Value from the specified AttributesMap, if available */
	static float GetCy(const FSVGAttributesMap& InAttributesMap);

	/** Get Radius Value from the specified AttributesMap, if available */
	static float GetR(const FSVGAttributesMap& InAttributesMap);

	/** Get Horizontal Coordinate Value from the specified AttributesMap, if available */
	static float GetX(const FSVGAttributesMap& InAttributesMap);

	/** Get Vertical Coordinate Value from the specified AttributesMap, if available */
	static float GetY(const FSVGAttributesMap& InAttributesMap);

	/** Get Width Value from the specified AttributesMap, if available */
	static float GetWidth(const FSVGAttributesMap& InAttributesMap);

	/** Get Height Value from the specified AttributesMap, if available */
	static float GetHeight(const FSVGAttributesMap& InAttributesMap);
};
