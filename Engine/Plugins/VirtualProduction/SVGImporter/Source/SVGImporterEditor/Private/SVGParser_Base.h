// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "Templates/SharedPointer.h"

struct FSVGRawElement;

#define LOCTEXT_NAMESPACE "SVGParser_Base"

/**
 * Base class used to implement specialized XML parsers to parse SVG Text Buffers.
 * Parser outputs the Root of a hierarchy of SVG Raw Elements.
 * From that root, it is possible to explore all Elements.
 */
class FSVGParser_Base : public TSharedFromThis<FSVGParser_Base>
{
public:
	FSVGParser_Base() = delete;

	/**
	 * Creates the SVG Parser
	 * @param InStringToParse the string to be parsed
	 */
	FSVGParser_Base(const FString& InStringToParse)
		:SVGString(InStringToParse)
	{}
	
	virtual ~FSVGParser_Base() = default;

	/**
	 * Parse the stored string. Returns true if parsing is successful.
	 * @param bInTryImportAnyway if parser fails, ask the user if they want to try importing the SVG anyway
	 */
	virtual bool Parse(bool bInTryImportAnyway = true) = 0;

	/** Returns true if the stored string is a valid SVG string (it has an SVG xml tag) */
	virtual bool IsValidSVG() = 0;
	
	/** Return the Root of the Parsed SVG Hierarchy. Root Element will be set after a successful parsing */
	const TSharedPtr<FSVGRawElement>& GetRootElement() { return RootSVGElement; }

protected:
	static EAppReturnType::Type ShowErrorMessageDialog(const FFormatNamedArguments& InArgs)
	{
		return FMessageDialog::Open(EAppMsgType::YesNo
			, FText::Format(LOCTEXT("SVGImportErrorMessage", "Error while parsing SVG text buffer {SVGFilename}:\n\n\t{SVGImportError}\n\nTry to import anyway?"), InArgs)
			, LOCTEXT("SVGImportErrorTitle", "SVG Editor SVG Importer"));
	}

	/** The string to be parsed */
	FString SVGString;
	
	/** The root of the SVG hierarchy, stored so that we can build it throughout XML/SVG parser callbacks */
	TSharedPtr<FSVGRawElement> RootSVGElement;

	/** Array containing all the SVG elements found by the parser */
	TArray<TSharedPtr<FSVGRawElement>> SVGElements;
};

#undef LOCTEXT_NAMESPACE
