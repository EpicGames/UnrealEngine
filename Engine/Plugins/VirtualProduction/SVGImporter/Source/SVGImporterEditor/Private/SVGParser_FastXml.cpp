// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGParser_FastXml.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "SVGImporterEditorModule.h"
#include "Types/SVGRawAttribute.h"
#include "Types/SVGRawElement.h"

bool FSVGParser_FastXml::Parse(bool bInTryToImportWithError /* = true */)
{
	if (SVGString.IsEmpty())
	{
		UE_LOGF(SVGImporterEditorLog, Warning, "Trying to parse empty SVG string.");
		return false;
	}
	
	FText ErrorMessage;
	int32 ErrorLineNumber;

	// Create root of SVG hierarchy
	RootSVGElement = MakeShared<FSVGRawElement>();
	SVGElements.Emplace(RootSVGElement);

	// Trying to sanitize the SVG file string
	SVGString.ReplaceInline(TEXT("\n"), TEXT(" "));
	SVGString.ReplaceInline(TEXT("\r"), TEXT(" "));

	const bool bParserSuccess = FFastXml::ParseXmlFile(this, TEXT(""), &SVGString[0], nullptr, false, false, ErrorMessage, ErrorLineNumber);
	bHasBeenParsed = true;
	
	if (bParserSuccess)
	{
		return true;
	}
	
	if (bCheckForSVGTagOnly)
	{
		return false;
	}

	const FString DialogErrorMessage = FString::Printf(TEXT("line=%d, %s"), ErrorLineNumber, *ErrorMessage.ToString());
	
	FFormatNamedArguments Args;
	Args.Add(TEXT("SVGImportError"), FText::FromString(DialogErrorMessage));
	
	const EAppReturnType::Type ImportAnyway = ShowErrorMessageDialog(Args);
	
	if (bInTryToImportWithError && ImportAnyway == EAppReturnType::Yes)
	{
		return true;
	}
	
	return false;
}

bool FSVGParser_FastXml::IsValidSVG()
{
	if (!bHasBeenParsed)
	{
		constexpr bool bTryToImportWithError = false;
		Parse(bTryToImportWithError);
	}

	return bHasSVGTag;
}

bool FSVGParser_FastXml::ProcessXmlDeclaration(const TCHAR *InElementData, int32 InXmlFileLineNumber)
{
	return true;
}

bool FSVGParser_FastXml::ProcessElement(const TCHAR *InElementName, const TCHAR *InElementData, int32 InXmlFileLineNumber)
{
	if (!bHasSVGTag)
	{
		bHasSVGTag = (FString(InElementName) == TEXT("svg"));
	}
	
	if (bCheckForSVGTagOnly && bHasSVGTag)
	{
		// No need to go on any further if we are only looking for the SVG tag, and we already found one.
		return false;
	}
	
	TSharedPtr<FSVGRawElement> CurrSVGElem = nullptr;

	if (SVGElements.Num() > 0)
	{
		CurrSVGElem = SVGElements.Last();
	}

	// An element has been found, let's create a new USVGParsedElement, specifying its parent, name and data
	TSharedRef<FSVGRawElement> NewElement = FSVGRawElement::NewSVGElement(CurrSVGElem, InElementName, InElementData);

	// If CurrSVGElem exists, then let's add the newly created element as one of its children
	if (CurrSVGElem)
	{
		CurrSVGElem->AddChild(NewElement);
	}

	// Add the new element to our list
	SVGElements.Emplace(NewElement);

	return true;
}

bool FSVGParser_FastXml::ProcessAttribute(const TCHAR *InAttributeName, const TCHAR *InAttributeValue)
{
	const TSharedPtr<FSVGRawElement>& CurrSVGElem = SVGElements.Last();

	if (!CurrSVGElem)
	{
		return false;
	}

	// Attribute already exists, returning
	if (CurrSVGElem->GetAttributes().Contains(InAttributeName))
	{
		return false;
	}

	// Creating the FSVGRawAttribute to hold the new attribute, and then adding the attribute to the currently parsed SVG element
	const TSharedRef<FSVGRawAttribute>& NewAttribute = FSVGRawAttribute::NewSVGAttribute(CurrSVGElem, InAttributeName, InAttributeValue);
	CurrSVGElem->AddAttribute(NewAttribute);

	return true;
}

bool FSVGParser_FastXml::ProcessClose(const TCHAR *Element)
{
	if (SVGElements.Num() == 0)
	{
		return false;
	}

	return true;
}
