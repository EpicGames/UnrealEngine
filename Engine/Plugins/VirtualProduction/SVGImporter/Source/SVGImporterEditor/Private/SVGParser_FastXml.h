// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FastXml.h"
#include "SVGParser_Base.h"

/**
 * SVG Parser based on FastXml parser
 */
class FSVGParser_FastXml : public FSVGParser_Base, public IFastXmlCallback
{
public:
	/**
	 * @param InStringToParse the string to be parsed
	 * @param bInCheckForSVGTagOnly if true, when the parser will run, it will only check for the presence of an SVG Tag.
	 * Useful to use the parser to quickly check if parsed string contains SVG info
	 */
	FSVGParser_FastXml(const FString& InStringToParse, bool bInCheckForSVGTagOnly = false)
		: FSVGParser_Base(InStringToParse)
		, bCheckForSVGTagOnly(bInCheckForSVGTagOnly)
	{
	}
	
	//~ Begin ISVGParser
	virtual bool Parse(bool bInTryToImportWithError = true) override;
	virtual bool IsValidSVG() override;
	//~ End ISVGParser

protected:
	//~ Begin IFastXmlCallback
	virtual bool ProcessXmlDeclaration(const TCHAR* InElementData, int32 InXmlFileLineNumber) override;
	virtual bool ProcessElement(const TCHAR* InElementName, const TCHAR* InElementData, int32 InXmlFileLineNumber) override;
	virtual bool ProcessAttribute(const TCHAR* InAttributeName, const TCHAR* InAttributeValue) override;
	virtual bool ProcessClose(const TCHAR* InElementName) override;
	virtual bool ProcessComment(const TCHAR* InComment) override { return true; }
	//~ End IFastXmlCallback

private:
	bool bHasBeenParsed = false;
	bool bCheckForSVGTagOnly = false;
	bool bHasSVGTag = false;
};
