// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGParser_PugiXml.h"
#include "Types/SVGRawAttribute.h"
#include "Types/SVGRawElement.h"

bool FSVGParser_PugiXml::Parse(bool bInTryToImportWithError /* = true */)
{
	if (!XmlDoc)
	{
		ParseXML();
	}

	if (XmlParseResult.status == pugi::xml_parse_status::status_ok)
	{
		// Create root of SVG hierarchy
		RootSVGElement = MakeShared<FSVGRawElement>();
		SVGElements.Emplace(RootSVGElement);

		if (const pugi::xml_node* DocPtr = XmlDoc.Get())
		{
			ParseNode(*DocPtr, RootSVGElement);
			return true;
		}
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("SVGImportError"), FText::FromString(XmlParseResult.description()));

		const EAppReturnType::Type ImportAnyway = ShowErrorMessageDialog(Args);
		if (bInTryToImportWithError && ImportAnyway == EAppReturnType::Yes)
		{
			return true;
		}
	}

	return false;
}

bool FSVGParser_PugiXml::IsValidSVG()
{
	if (!XmlDoc)
	{
		ParseXML();
	}

	return XmlParseResult.status == pugi::xml_parse_status::status_ok;
}

void FSVGParser_PugiXml::ParseXML()
{
	if (!XmlDoc)
	{
		XmlDoc = MakeShared<pugi::xml_document>();
	}

	// Parse xml from buffer
	XmlParseResult = XmlDoc->load_buffer(GetData(SVGString), SVGString.GetAllocatedSize());
}

void FSVGParser_PugiXml::ParseNode(const pugi::xml_node& InNode, const TSharedPtr<FSVGRawElement>& InParent)
{
	const TSharedPtr<FSVGRawElement> ParentElement = InParent.IsValid() ? InParent : RootSVGElement;

	const pugi::char_t* NodeName = InNode.name();

	// node.child_value() is the way for elements to get non attributes values.
	// For SVGs this can be found e.g in style tags: <style> VALUE </style>
	const pugi::char_t* NodeValue = InNode.child_value();

	// An element has been found, let's create a new USVGParsedElement, specifying its parent, name and data
	const TSharedRef<FSVGRawElement> NewElement = FSVGRawElement::NewSVGElement(ParentElement, NodeName, NodeValue);

	if (ParentElement)
	{
		ParentElement->AddChild(NewElement);
	}

	for (pugi::xml_attribute_iterator AttrItr = InNode.attributes_begin(); AttrItr != InNode.attributes_end(); ++AttrItr)
	{
		// Attribute already exists, continue
		if (NewElement->GetAttributes().Contains(AttrItr->name()))
		{
			continue;
		}

		// Creating the FSVGRawAttribute to hold the new attribute, and then adding the attribute to the currently parsed SVG element
		const TSharedRef<FSVGRawAttribute> NewAttribute = FSVGRawAttribute::NewSVGAttribute(NewElement, AttrItr->name(), AttrItr->value());
		NewElement->AddAttribute(NewAttribute);
	}

	for (const pugi::xml_node& ChildNode : InNode.children())
	{
		ParseNode(ChildNode, NewElement);
	}
}
