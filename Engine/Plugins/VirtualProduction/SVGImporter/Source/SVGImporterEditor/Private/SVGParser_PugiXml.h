// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGParser_Base.h"

#include "pugixml.hpp"

/**
 * SVG Parser based on pugi xml parser
 */
class FSVGParser_PugiXml : public FSVGParser_Base
{
public:
	FSVGParser_PugiXml(const FString& InStringToParse)
		: FSVGParser_Base(InStringToParse)
	{}

	//~ Begin ISVGParser
	virtual bool Parse(bool bInTryToImportWithError = true) override;
	virtual bool IsValidSVG() override;
	//~ Begin ISVGParser

protected:
	/** Parse the XML text buffer. This will not populate the SVG hierarchy yet. */
	void ParseXML();

	/** Recursively parse a node and its children nodes */
	void ParseNode(const pugi::xml_node& InNode, const TSharedPtr<FSVGRawElement>& InParent);

	/** The pugi xml document which will be used for parsing */
	TSharedPtr<pugi::xml_document> XmlDoc;

	/** Result of xml parsing */
	pugi::xml_parse_result XmlParseResult;
};
