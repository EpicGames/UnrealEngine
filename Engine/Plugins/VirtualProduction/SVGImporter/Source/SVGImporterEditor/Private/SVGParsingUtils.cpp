// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGParsingUtils.h"
#include "SVGDefines.h"
#include "SVGImporterEditorModule.h"
#include "SVGParser_FastXml.h"
#include "SVGParser_PugiXml.h"
#include "SVGPath.h"
#include "Types/SVGRawAttribute.h"

namespace UE::SVGImporterEditor::Private
{
	constexpr double SVG_PI = 3.14159265358979323846264338327f;
	constexpr double SVGScaleFactor = 0.1f;
}

// This function is a custom parser to handle decimal runs written as 23.45.02
// which are to be interpreted as something like 23.45, 0.02
int32 ParseDecimals(TArray<FString>& OutArray, const FString& Data, bool InCullEmpty)
{
	// Make sure the delimit string is not null or empty
	OutArray.Reset();
	const TCHAR *Start = Data.GetCharArray().GetData();
	const int32 Length = Data.Len();

	if (Start)
	{
		int32 CurrFloatBeginIndex = 0;
		int32 CurrDotIndex = INDEX_NONE;

		// Iterate through string.
		for (int32 i = 0; i < Length; i++)
		{
			const TCHAR* Delimiter = TEXT(".");
			const int32 DelimiterLength = FCString::Strlen(Delimiter);

			// If we found a dot
			if (FCString::Strncmp(Start + i, Delimiter, DelimiterLength) == 0)
			{
				// If we already found one before, then we just finished parsing a float
				if (CurrDotIndex != INDEX_NONE)
				{
					const int32 CurrFloatEndIndex = i - 1;
					OutArray.Add(FString::ConstructFromPtrSize(Start + CurrFloatBeginIndex, CurrFloatEndIndex - CurrFloatBeginIndex));

					// go for the next one
					CurrFloatBeginIndex = i;
				}

				CurrDotIndex = i;
			}
		}

		// Add any remaining characters after the last delimiter.
		const int32 SubstringLength = Length - CurrFloatBeginIndex;
		// If we're not culling empty strings or if we are but the string isn't empty anyways...
		if (!InCullEmpty || SubstringLength != 0)
		{
			// ... add new string from substring beginning up to the beginning of this delimiter.
			OutArray.Add(Start + CurrFloatBeginIndex);
		}
	}

	return OutArray.Num();
}

void FSVGParsingUtils::PointsFromString(FString InPointsString, TArray<FVector2D>& OutPoints)
{
	OutPoints.Empty();

	constexpr bool bApplyScale = true;
	const TArray<float> Points = FloatsFromString(InPointsString, bApplyScale);

	if (Points.Num() % 2 != 0)
	{
		UE_LOGF(SVGImporterEditorLog, Warning, "Trying to parse SVG attribute as point list, but number of resulting members is not even %i. Returned points list will be empty. %ls", Points.Num(), *InPointsString);
		return;
	}

	OutPoints.Reserve(Points.Num() / 2);
    for (int32 PointIndex = 0; PointIndex + 1 < Points.Num(); PointIndex += 2)
    {
        OutPoints.Add(FVector2D(Points[PointIndex], Points[PointIndex + 1]));
    }
}

TSharedRef<FSVGParser_Base> FSVGParsingUtils::CreateSVGParser(const FString& InStringToParse, ESVGParserType InParserType)
{
	switch (InParserType)
	{
		case ESVGParserType::SVGFastXml:
			return MakeShared<FSVGParser_FastXml>(InStringToParse);

		case ESVGParserType::SVGPugiXml:
		default:
			return MakeShared<FSVGParser_PugiXml>(InStringToParse);
	}
}

bool FSVGParsingUtils::IsValidSVGString(const FString& InStringToParse)
{
	const TSharedRef<FSVGParser_Base> SVGParser = FSVGParsingUtils::CreateSVGParser(InStringToParse);
	return SVGParser->IsValidSVG();
}

float FSVGParsingUtils::FloatFromString(const FString& InFloatString)
{
	return FCString::Atof(*InFloatString) * UE::SVGImporterEditor::Private::SVGScaleFactor;
}

bool FSVGParsingUtils::FloatFromStringWithSuffix(const FString& InFloatString, const FString& InSuffix, float& OutValue)
{
	if (InFloatString.EndsWith(InSuffix))
	{
		FString ReturnValue = InFloatString;
		ReturnValue.RemoveFromEnd(InSuffix);

		OutValue = FCString::Atof(*ReturnValue);
		return true;
	}
	else
	{
		OutValue = 0.0f;
		return false;
	}
}

TArray<float> FSVGParsingUtils::FloatsFromString(FString InFloatsString, bool bApplyScaling /*default == true*/)
{
	InFloatsString.TrimStartAndEndInline();
	
    TArray<float> ParsedValues;
    FString CurrentNumber;
    bool bIsScientificNotation = false;
	bool bHasDecimalPoint = false;
	const float Multiplier = bApplyScaling ? UE::SVGImporterEditor::Private::SVGScaleFactor : 1.f;

	/**
	 * Formats supported
	 * 10,20 => (10, 20)
	 * 10 20 => (10, 20)
	 * -3.48-1.92-.53.66 => (-3.48,-1.92),(-0.53,0.66)
	 * .62.78,3.76-2.11 => (0.62,0.78),(3.76,-2.11)
	 * .10, .20 => (0.10,0.20)
	 * .10 .20 => (0.10,0.20)
	 * .62e3.78E-1 => (0.62e3,0.78e-1)
	 */
    for (int32 CharIndex = 0; CharIndex < InFloatsString.Len(); ++CharIndex)
    {
	    const TCHAR Char = InFloatsString[CharIndex];

        // Check if character is part of a number : -.10
        if (FChar::IsDigit(Char) || (Char == '.' && !bHasDecimalPoint) || (CurrentNumber.IsEmpty() && (Char == '-' || Char == '+')))
        {
            CurrentNumber += Char;

        	if (Char == '.')
        	{
        		bHasDecimalPoint = true;
        	}
        }
    	// Scientific notation detected; mark the flag and include 'e'/'E' : -10e3
        else if ((Char == 'e' || Char == 'E') && !CurrentNumber.IsEmpty())
        {
            bIsScientificNotation = true;
            CurrentNumber += Char;
        }
    	// Scientific notation exponent sign : -10e3
        else if ((Char == '-' || Char == '+') && bIsScientificNotation)
        {
            CurrentNumber += Char;
            bIsScientificNotation = false;
        }
    	// non-numeric character encountered
        else
        {
            if (!CurrentNumber.IsEmpty())
            {
                ParsedValues.Add(FCString::Atof(*CurrentNumber) * Multiplier);
            }

        	// Reset
        	CurrentNumber.Empty();
            bIsScientificNotation = false;
        	bHasDecimalPoint = false;

            // Skip comma as a delimiter
            if (Char == ',')
            {
                continue;
            }

            // Start next number
            if (Char == '-' || Char == '+' || Char == '.')
            {
                CurrentNumber += Char;
            	
            	if (Char == '.')
            	{
            		bHasDecimalPoint = true;
            	}
            }
        }
    }

    if (!CurrentNumber.IsEmpty())
    {
        ParsedValues.Add(FCString::Atof(*CurrentNumber) * Multiplier);
    }

	return ParsedValues;
}

int32 FSVGParsingUtils::ParseIntoCommandsArrayKeepDelimiters(const FString& InSourceString, TArray<FString>& OutArray, const TCHAR* const * DelimArray, int32 NumDelims, bool InCullEmpty)
{
	// Make sure the delimit string is not null or empty
	check(DelimArray);
	OutArray.Reset();

	if (const TCHAR *Start = *InSourceString)
	{
		// Iterate through string.
		for (int32 i = 0; i < InSourceString.Len();)
		{
			int32 CmdBeginIndex = INDEX_NONE;
			int32 CmdEndIndex = INDEX_NONE;

			int32 DelimiterLength = 0;

			// Attempt each delimiter.
			for (int32 DelimIndex = 0; DelimIndex < NumDelims; ++DelimIndex)
			{
				DelimiterLength = FCString::Strlen(DelimArray[DelimIndex]);

				// If we found a delimiter...
				if (FCString::Strncmp(Start + i, DelimArray[DelimIndex], DelimiterLength) == 0)
				{
					// Mark the beginning of the substring.
					CmdBeginIndex = i;
					break;
				}
			}

			for (int32 j = CmdBeginIndex + DelimiterLength; j < InSourceString.Len(); j++)
			{
				bool bFound = false;
				// Attempt each delimiter.
				for (int32 NextDelimIndex = 0; NextDelimIndex < NumDelims; ++NextDelimIndex)
				{
					const int32 NextDelimiterLength = FCString::Strlen(DelimArray[NextDelimIndex]);

					// If we found a delimiter...
					if (FCString::Strncmp(Start + j, DelimArray[NextDelimIndex], NextDelimiterLength) == 0)
					{
						// Mark the end of the substring (subtract length of delimiter).
						CmdEndIndex = j - (NextDelimiterLength - 1);
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					break;
				}
			}

			if (CmdBeginIndex != INDEX_NONE)
			{
				if (CmdEndIndex == INDEX_NONE)
				{
					CmdEndIndex = InSourceString.Len();
				}

				const int32 CmdStringLength = CmdEndIndex - CmdBeginIndex;

				if (CmdStringLength != 0)
				{
					OutArray.Add(FString::ConstructFromPtrSize(Start + CmdBeginIndex, CmdEndIndex - CmdBeginIndex));
				}

				// Next cmd begins at the end of this cmd
				CmdBeginIndex = CmdEndIndex;
				i = CmdBeginIndex;
			}
			else
			{
				i++;
			}
		}
	}

	return OutArray.Num();
}

TArray<TArray<FSVGPathCommand>> FSVGParsingUtils::ParseStringAsPathCommands(const FString& InString)
{
	// Will collect the sub paths found in this command
	TArray<TArray<FSVGPathCommand>> SubPaths;

	if (!(InString.StartsWith("m") || InString.StartsWith("M")))
	{
		UE_LOGF(SVGImporterEditorLog, Warning, "Trying to parse SVG attribute as path, but first instruction is not a moveto! Returned path commands list will be empty.");
		return SubPaths;
	}

	// default array of LineEndings
	static const TCHAR* Separators[] =
	{
		TEXT("m"), // move to
		TEXT("M"),
		TEXT("z"), // close path
		TEXT("Z"),
		TEXT("l"), // line to
		TEXT("L"),
		TEXT("h"), // horizontal line to
		TEXT("H"),
		TEXT("v"), // vertical line to
		TEXT("V"),
		TEXT("c"), // curve to
		TEXT("C"),
		TEXT("s"), // smooth curve to
		TEXT("S"),
		TEXT("Q"), // quadratic curve to
		TEXT("q"),
		TEXT("t"), // smooth quadratic curve to
		TEXT("T"),
		TEXT("a"), // elliptical arc
		TEXT("A"),
	};

	// start with just the standard line endings
	constexpr int32 SeparatorsNum = UE_ARRAY_COUNT(Separators);

	TArray<FString> CommandsToParse;
	ParseIntoCommandsArrayKeepDelimiters(InString, CommandsToParse, Separators, SeparatorsNum, true);

	int32 CurrPathIndex = -1;

	FVector2D InitialPoint = FVector2D::ZeroVector;
	FVector2D CursorPos = FVector2D::ZeroVector;
	FVector2D CursorPos2 = FVector2D::ZeroVector;

	for (FString& CommandStringElem : CommandsToParse)
	{
		FString CommandString = CommandStringElem;

		TCHAR CommandType = CommandString.GetCharArray()[0];
		CommandString.RemoveAt(0);

		bool bInvalidCommand = false;

		switch (CommandType)
		{
			case 'm':
			case 'M':
				{
					TArray<FVector2D> MPoints;
					PointsFromString(CommandString, MPoints);
					const bool bIsRelative = CommandType == 'm';

					if (MPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					// moveto parameters: (x y)+
					// move to implies the start of a new sub path, let's create it
					TArray<FSVGPathCommand> NewSubPath;
					SubPaths.Add(NewSubPath);
					CurrPathIndex++;

					// We store the initial point of the sub path to properly apply closepath later, if needed
					InitialPoint = PathMoveTo(SubPaths[CurrPathIndex], CursorPos, MPoints, bIsRelative);
					CursorPos2 = InitialPoint;
				}
			break;

			case 'z':
			case 'Z':
				{
					if (SubPaths.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}
					
					// closepath parameters: none
					FSVGPathCommand ClosePathCmd(ESVGPathInstructionType::ClosePath);
					ClosePathCmd.PointTo = InitialPoint;
					SubPaths.Last().Add(ClosePathCmd);
					CursorPos = ClosePathCmd.PointTo;
					CursorPos2 = CursorPos;
				}
			break;

			case 'l':
			case 'L':
				{
					// lineto parameters: (x y)+
					TArray<FVector2D> LPoints;
					PointsFromString(CommandString, LPoints);
					const bool bIsRelative = CommandType == 'l';

					if (LPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathLineTo(SubPaths[CurrPathIndex], CursorPos, LPoints, bIsRelative);
					CursorPos2 = CursorPos;
				}
				break;

			case 'h':
			case 'H':
				{
					// horizontal lineto parameters: x+
					TArray<float> HFloats = FloatsFromString(CommandString);
					const bool bIsRelative = CommandType == 'h';

					if (HFloats.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathHorizontalLineTo(SubPaths[CurrPathIndex], CursorPos, HFloats, bIsRelative);
					CursorPos2 = CursorPos;
				}
				break;

			case 'v':
			case 'V':
				{
					// vertical lineto parameters: y+
					TArray<float> VFloats = FloatsFromString(CommandString);
					const bool bIsRelative = CommandType == 'v';

					if (VFloats.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}
					
					PathVerticalLineTo(SubPaths[CurrPathIndex], CursorPos, VFloats, bIsRelative);
					CursorPos2 = CursorPos;
				}
				break;

			case 'c':
			case 'C':
				{
					// curveto parameters: (x1 y1 x2 y2 x y)+
					TArray<FVector2D> CPoints;
					PointsFromString(CommandString, CPoints);
					const bool bIsRelative = CommandType == 'c';

					if (CPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathCubicBezierTo(SubPaths[CurrPathIndex], CursorPos, CursorPos2, CPoints, bIsRelative);
				}
				break;

			case 's':
			case 'S':
				{
					// smooth curveto parameters: (x2 y2 x y)+
					TArray<FVector2D> SPoints;
					PointsFromString(CommandString, SPoints);
					const bool bIsRelative = CommandType == 's';

					if (SPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathCubicBezierSmoothTo(SubPaths[CurrPathIndex], CursorPos, CursorPos2, SPoints, bIsRelative);
				}
				break;

			case 'Q':
			case 'q':
				{
					// quadratic bezier curveto parameters: (x1 y1 x y)+
					TArray<FVector2D> QPoints;
					PointsFromString(CommandString, QPoints);
					const bool bIsRelative = CommandType == 'q';
					
					if (QPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathQuadraticBezierTo(SubPaths[CurrPathIndex], CursorPos, CursorPos2, QPoints, bIsRelative);
				}
				break;

			case 't':
			case 'T':
				{
					// smooth quadratic bezier curveto parameters: (x y)+
					TArray<FVector2D> TPoints;
					PointsFromString(CommandString, TPoints);
					const bool bIsRelative = CommandType == 't';

					if (TPoints.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}

					PathQuadraticBezierSmoothTo(SubPaths[CurrPathIndex], CursorPos, CursorPos2, TPoints, bIsRelative);
				}
				break;

			case 'a':
			case 'A':
				{
					// elliptical arc parameters: (rx ry x-axis-rotation large-arc-flag sweep-flag x y)+
					bool bApplyScaling = false;// we don't want to scale everything, since arcs have angles and flags inside
					TArray<float> AValues = FloatsFromString(CommandString, bApplyScaling);
					bool bIsRelative = CommandType == 'a';

					if (AValues.IsEmpty())
					{
						bInvalidCommand = true;
						break;
					}
					
					PathArcTo(SubPaths[CurrPathIndex], CursorPos, AValues, bIsRelative);
					CursorPos2 = CursorPos;
				}
			break;

			default:
			break;
		}

		if (bInvalidCommand)
		{
			UE_LOGF(SVGImporterEditorLog, Warning, "Invalid svg command detected during parsing (%ls), skipping entire svg subpath (%ls)", *CommandStringElem, *InString);
			SubPaths.Empty();
			break;
		}
	}

	return SubPaths;
}

bool FSVGParsingUtils::IsURLString(const FString& InString)
{
	using namespace UE::SVGImporter::Public;

	return InString.StartsWith(SVGConstants::URL_Start) && InString.EndsWith(SVGConstants::URL_End);
}

FString FSVGParsingUtils::ExtractURL(const FString& InString)
{
	using namespace UE::SVGImporter::Public;

	FString OutString = InString;

	if (IsURLString(InString))
	{
		OutString.RemoveFromStart(SVGConstants::URL_Start);
		OutString.RemoveFromEnd(SVGConstants::URL_End);
	}

	return OutString;
}

bool FSVGParsingUtils::IsVisible(const TSharedRef<FSVGRawAttribute>& InAttribute)
{
	const FString& DisplayString = InAttribute->AsString();
	return DisplayString != UE::SVGImporter::Public::SVGConstants::None;
}

FVector2D FSVGParsingUtils::PathMoveTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	FVector2D InitialPosition = FVector2D::ZeroVector;

	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		if (bInIsRelative)
		{
			InOutCursorPos += InArguments[i];
		}
		else
		{
			InOutCursorPos = InArguments[i];
		}

		// First xy pair is MoveTo
		if (i == 0)
		{
			FSVGPathCommand MoveToCmd(ESVGPathInstructionType::MoveTo);
			MoveToCmd.PointTo = InOutCursorPos;
			OutCommands.Add(MoveToCmd);
			InitialPosition = MoveToCmd.PointTo;
		}
		// Following pairs are treated as LineTos
		else
		{
			FSVGPathCommand LineToCmd(ESVGPathInstructionType::LineTo);
			LineToCmd.PointTo = InOutCursorPos;
			OutCommands.Add(LineToCmd);
		}
	}

	return InitialPosition;
}

void FSVGParsingUtils::PathLineTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	// We could have multiple xy pairs
	for (const FVector2D& Point : InArguments)
	{
		if (bInIsRelative)
		{
			InOutCursorPos += Point;
		}
		else
		{
			InOutCursorPos = Point;
		}

		FSVGPathCommand LineToCmd(ESVGPathInstructionType::LineTo);
		LineToCmd.PointTo = InOutCursorPos;
		OutCommands.Add(LineToCmd);
	}
}

void FSVGParsingUtils::PathHorizontalLineTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	const TArray<float>& InArguments, bool bInIsRelative)
{
	for (const float& PointX : InArguments)
	{
		if (bInIsRelative)
		{
			InOutCursorPos.X += PointX;
		}
		else
		{
			InOutCursorPos.X = PointX;
		}

		FSVGPathCommand LineToCmd(ESVGPathInstructionType::LineTo);
		LineToCmd.PointTo = InOutCursorPos;
		OutCommands.Add(LineToCmd);
	}
}

void FSVGParsingUtils::PathVerticalLineTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	const TArray<float>& InArguments, bool bInIsRelative)
{
	for (const float& PointY : InArguments)
	{
		if (bInIsRelative)
		{
			InOutCursorPos.Y += PointY;
		}
		else
		{
			InOutCursorPos.Y = PointY;
		}

		FSVGPathCommand LineToCmd(ESVGPathInstructionType::LineTo);
		LineToCmd.PointTo = InOutCursorPos;
		OutCommands.Add(LineToCmd);
	}
}

void FSVGParsingUtils::PathCubicBezierTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	FVector2D& InOutCursorPos2, const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	// cubic beziers come in sets of 3 xy pairs, and they can be concatenated
	if (InArguments.Num()%3 == 0)
	{
		for (int32 i = 0; i < InArguments.Num(); i+=3)
		{
			FVector2D CP1;
			FVector2D CP2;
			FVector2D PointTo;

			if (bInIsRelative)
			{
				CP1 = InOutCursorPos + InArguments[i + 0];
				CP2 = InOutCursorPos + InArguments[i + 1];
				PointTo = InOutCursorPos + InArguments[i + 2];
			}
			else
			{
				CP1 = InArguments[i + 0];
				CP2 = InArguments[i + 1];
				PointTo = InArguments[i + 2];
			}

			FSVGPathCommand CurveToCmd(ESVGPathInstructionType::CurveTo);
			CurveToCmd.ArriveControlPoint = CP1;
			CurveToCmd.LeaveControlPoint = CP2;
			CurveToCmd.PointTo = PointTo;
			OutCommands.Add(CurveToCmd);

			InOutCursorPos2 = CP2;
			InOutCursorPos = PointTo;
		}
	}
}

void FSVGParsingUtils::PathCubicBezierSmoothTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	FVector2D& InOutCursorPos2, const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	if (InArguments.Num()%2 == 0)
	{
		for (int32 i = 0; i < InArguments.Num(); i+=2)
		{
			FVector2D CP1;
			FVector2D CP2;
			FVector2D PointFrom;
			FVector2D PointTo;

			PointFrom = InOutCursorPos;

			if (bInIsRelative)
			{
				CP2 = InOutCursorPos + InArguments[i + 0];
				PointTo = InOutCursorPos + InArguments[i + 1];
			}
			else
			{
				CP2 = InArguments[i + 0];
				PointTo = InArguments[i + 1];
			}

			CP1 = 2.0f*PointFrom - InOutCursorPos2;

			FSVGPathCommand CurveToCmd(ESVGPathInstructionType::CurveTo);
			CurveToCmd.ArriveControlPoint = CP1;
			CurveToCmd.LeaveControlPoint = CP2;
			CurveToCmd.PointTo = PointTo;
			OutCommands.Add(CurveToCmd);

			InOutCursorPos2 = CP2;
			InOutCursorPos = PointTo;
		}
	}
}

void FSVGParsingUtils::PathQuadraticBezierTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	FVector2D& InOutCursorPos2, const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	if (InArguments.Num()%2 == 0)
	{
		for (int32 i = 0; i < InArguments.Num(); i+=2)
		{
			FVector2D CP1;
			FVector2D CP2;
			FVector2D PointFrom;
			FVector2D PointTo;
			FVector2D CPQuad;

			PointFrom = InOutCursorPos;	

			if (bInIsRelative)
			{
				CPQuad = InOutCursorPos + InArguments[i + 0];
				PointTo = InOutCursorPos + InArguments[i + 1];
			}
			else
			{
				CPQuad = InArguments[i + 0];
				PointTo = InArguments[i + 1];
			}

			// Convert to cubic bezier
			CP1 = PointFrom + 2.0f/3.0f*(CPQuad - PointFrom);
			CP2 = PointTo + 2.0f/3.0f*(CPQuad - PointTo);

			FSVGPathCommand CurveToCmd(ESVGPathInstructionType::CurveTo);
			CurveToCmd.ArriveControlPoint = CP1;
			CurveToCmd.LeaveControlPoint = CP2;
			CurveToCmd.PointTo = PointTo;
			OutCommands.Add(CurveToCmd);

			InOutCursorPos2 = CPQuad;
			InOutCursorPos = PointTo;
		}
	}
}

void FSVGParsingUtils::PathQuadraticBezierSmoothTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	FVector2D& InOutCursorPos2, const TArray<FVector2D>& InArguments, bool bInIsRelative)
{
	for (const FVector2D& Point : InArguments)
	{
		FVector2D CP1;
		FVector2D CP2;
		FVector2D PointFrom;
		FVector2D PointTo;
		FVector2D CPQuad;

		PointFrom = InOutCursorPos;	

		if (bInIsRelative)
		{
			PointTo = InOutCursorPos + Point;
		}
		else
		{
			PointTo = Point;
		}

		CPQuad = 2*PointFrom - InOutCursorPos2;

		// Convert to cubic bezier
		CP1 = PointFrom + 2.0f/3.0f*(CPQuad - PointFrom);
		CP2 = PointTo + 2.0f/3.0f*(CPQuad - PointTo);

		FSVGPathCommand CurveToCmd(ESVGPathInstructionType::CurveTo);
		CurveToCmd.ArriveControlPoint = CP1;
		CurveToCmd.LeaveControlPoint = CP2;
		CurveToCmd.PointTo = PointTo;
		OutCommands.Add(CurveToCmd);

		InOutCursorPos2 = CPQuad;
		InOutCursorPos = PointTo;
	}
}

static float SVGAngle(const FVector2D& U, const FVector2D& V)
{
	const float Dot = FVector2D::DotProduct(U, V);
	const float Len = U.Length()*V.Length();
	float Angle = FMath::Acos(FMath::Clamp(Dot/Len, -1.0f, 1.0f));

	if (U.X* V.Y - U.Y*V.X < 0.0f)
	{
		Angle = -Angle;
	}

	return Angle;
}

static void XFormPoint(float& Dx, float& Dy, float X, float Y, const TArray<float>& T)
{
	if (T.Num() < 6)
	{
		return;
	}

	Dx = X*T[0] + Y*T[2] + T[4];
	Dy = X*T[1] + Y*T[3] + T[5];
}

static void XFormVec(float& Dx, float& Dy, float X, float Y, const TArray<float>& T)
{
	if (T.Num() < 4)
	{
		return;
	}

	Dx = X*T[0] + Y*T[2];
	Dy = X*T[1] + Y*T[3];
}

void FSVGParsingUtils::PathArcTo(TArray<FSVGPathCommand>& OutCommands, FVector2D& InOutCursorPos,
	const TArray<float>& InArguments, bool bInIsRelative)
{
	// elliptical arc should have 7 params: rx, ry, x-rot, large-arc-flag, sweep-flag, x, y
	if (InArguments.Num()%7 == 0)
	{
		int32 ArcsNum = InArguments.Num()/7;
		for (int32 k = 0; k < ArcsNum; k++)
		{
			float Rx = FMath::Abs(InArguments[k*7 + 0]) * UE::SVGImporterEditor::Private::SVGScaleFactor;
			float Ry = FMath::Abs(InArguments[k*7 + 1]) * UE::SVGImporterEditor::Private::SVGScaleFactor;
			float RotX = (InArguments[k*7 + 2] * UE::SVGImporterEditor::Private::SVG_PI)/180.0f;	// rotation angle, from deg to rad
			int Fa = FMath::Abs(InArguments[k*7 + 3]) > 1e-6 ? 1 : 0;	// large arc
			int Fs = FMath::Abs(InArguments[k*7 + 4]) > 1e-6 ? 1 : 0;	// sweep direction

			FVector2D PointFrom = InOutCursorPos;
			FVector2D PointTo;

			if (bInIsRelative)
			{
				PointTo = InOutCursorPos + FVector2D(InArguments[k*7 + 5], InArguments[k*7 + 6])*UE::SVGImporterEditor::Private::SVGScaleFactor;
			}
			else
			{
				PointTo = FVector2D(InArguments[k*7 + 5], InArguments[k*7 + 6])*UE::SVGImporterEditor::Private::SVGScaleFactor;
			}

			float Dx = PointFrom.X - PointTo.X;
			float Dy = PointFrom.Y - PointTo.Y;
			float D = FMath::Sqrt(Dx*Dx + Dy*Dy);

			if (D < 1e-6f || Rx < 1e-6f || Ry < 1e-6f)
			{
				// for such small values, arcs degenerates to lines!
				FSVGPathCommand LineToCmd(ESVGPathInstructionType::LineTo);
				LineToCmd.PointTo = PointTo;
				OutCommands.Add(LineToCmd);
				InOutCursorPos = PointTo;
				return;
			}

			float SinRot = FMath::Sin(RotX);
			float CosRot = FMath::Cos(RotX);

			// Convert to center point parameterization.
			// http://www.w3.org/TR/SVG11/implnote.html#ArcImplementationNotes
			// 1) Compute x1', y1'
			float X1p =  CosRot * Dx / 2.0f + SinRot * Dy / 2.0f;
			float Y1p = -SinRot * Dx / 2.0f + CosRot * Dy / 2.0f;

			D = FMath::Square(X1p)/FMath::Square(Rx) + FMath::Square(Y1p)/FMath::Square(Ry);
			if (D > 1.0f)
			{
				D = FMath::Sqrt(D);
				Rx *= D;
				Ry *= D;
			}

			// 2) Compute cx', cy'
			float S = 0.0f;
			float Sa = FMath::Square(Rx) * FMath::Square(Ry)  - FMath::Square(Rx) * FMath::Square(Y1p) - FMath::Square(Ry)*FMath::Square(X1p);
			float Sb = FMath::Square(Rx) * FMath::Square(Y1p) + FMath::Square(Ry) * FMath::Square(X1p);

			if (Sa < 0.0f)
			{
				Sa = 0.0f;
			}

			if (Sb > 0.0f)
			{
				S = FMath::Sqrt(Sa / Sb);
			}

			if (Fa == Fs)
			{
				S = -S;
			}

			float Cxp = S * Rx * Y1p / Ry;
			float Cyp = S * -Ry * X1p / Rx;

			// 3) Compute cx,cy from cx',cy'
			float Cx = (PointFrom.X + PointTo.X)/2.0f + CosRot*Cxp - SinRot*Cyp;
			float Cy = (PointFrom.Y + PointTo.Y)/2.0f + SinRot*Cxp + CosRot*Cyp;

			// 4) Calculate theta1, and delta theta.
			FVector2D U = FVector2D((X1p - Cxp) / Rx, (Y1p - Cyp) / Ry);
			FVector2D V = FVector2D((-X1p - Cxp) / Rx, (-Y1p - Cyp) / Ry);

			float A1 = SVGAngle(FVector2D(1.0f, 0.0f), U);	// Initial angle
			float Da = SVGAngle(U, V);		// Delta angle

			if (Fs == 0 && Da > 0.0f)
			{
				Da -= 2.0f * UE::SVGImporterEditor::Private::SVG_PI;
			}
			else if (Fs == 1 && Da < 0.0f)
			{
				Da += 2.0f * UE::SVGImporterEditor::Private::SVG_PI;
			}

			TArray<float> T;
			T.AddZeroed(6);

			// Approximate the arc using cubic spline segments.
			T[0] = CosRot;
			T[1] = SinRot;
			T[2] = -SinRot;
			T[3] = CosRot;
			T[4] = Cx;
			T[5] = Cy;

			// Split arc into max 90 degree segments.
			// The loop assumes an iteration per end point (including start and end), this +1.
			int NumDivs = static_cast<int>(FMath::Abs(Da) / (UE::SVGImporterEditor::Private::SVG_PI * 0.5f) + 1.0f);
			float HDa = (Da / NumDivs) / 2.0f;

			// handle small values for cotangents
			if (HDa < 1e-3f && HDa > -1e-3f)
			{
				HDa *= 0.5f;
			}
			else
			{
				HDa = (1.0f - FMath::Cos(HDa)) / FMath::Sin(HDa);
			}

			float Kappa = FMath::Abs(4.0f / 3.0f * HDa);

			if (Da < 0.0f)
			{
				Kappa = -Kappa;
			}

			float A;
			float X;
			float Y;

			float PrevX = 0.0f;
			float PrevY = 0.0f;
			float PrevTanX = 0.0f;
			float PrevTanY = 0.0f;
			float TanX = 0.0f;
			float TanY = 0.0f;

			for (int i = 0; i <= NumDivs; i++)
			{
				A = A1 + Da * (static_cast<float>(i)/static_cast<float>(NumDivs));
				Dx = FMath::Cos(A);
				Dy = FMath::Sin(A);
				XFormPoint(X, Y, Dx*Rx, Dy*Ry, T); // position
				XFormVec(TanX, TanY, -Dy*Rx*Kappa, Dx*Ry*Kappa, T); // tangent

				if (i > 0)
				{
					FSVGPathCommand CurveFromEllipticalArcCmd(ESVGPathInstructionType::CurveTo);
					CurveFromEllipticalArcCmd.ArriveControlPoint =  FVector2D(PrevX + PrevTanX, PrevY + PrevTanY);
					CurveFromEllipticalArcCmd.LeaveControlPoint =  FVector2D(X-TanX, Y-TanY);
					CurveFromEllipticalArcCmd.PointTo = FVector2D(X,Y);

					OutCommands.Add(CurveFromEllipticalArcCmd);
				}

				PrevX = X;
				PrevY = Y;
				PrevTanX = TanX;
				PrevTanY = TanY;
			}

			InOutCursorPos = PointTo;
		}
	}
}
