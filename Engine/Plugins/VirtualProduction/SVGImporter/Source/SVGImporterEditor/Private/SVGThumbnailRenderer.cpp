// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGThumbnailRenderer.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Texture2D.h"
#include "RenderGraphBuilder.h"
#include "SVGData.h"
#include "TextureResource.h"
#include "ThumbnailRendering/ThumbnailManager.h"

bool USVGThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	const USVGData* SVGDataAsset = Cast<USVGData>(Object);

	if (IsValid(SVGDataAsset))
	{
		return true;
	}

	return false;
}

void USVGThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height,
	FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	const USVGData* SVGDataAsset = Cast<USVGData>(Object);

	if (SVGDataAsset->SVGTexture)
	{
		constexpr int32 CheckerDensity = 0;
		const TObjectPtr<UTexture2D> Checker = UThumbnailManager::Get().CheckerboardTexture;
		Canvas->DrawTile(
			0.0f, 0.0f, Width, Height, // Dimensions
			0.0f, 0.0f, CheckerDensity, CheckerDensity, // UVs
			FLinearColor::White, Checker->GetResource()); // Tint & Texture
		
		FCanvasTileItem CanvasTile(FVector2D(X, Y), SVGDataAsset->SVGTexture->GetResource(), FVector2D(Width, Height), FLinearColor::White);
		CanvasTile.BlendMode = SE_BLEND_AlphaBlend;
		CanvasTile.Draw(Canvas);
	}
}
