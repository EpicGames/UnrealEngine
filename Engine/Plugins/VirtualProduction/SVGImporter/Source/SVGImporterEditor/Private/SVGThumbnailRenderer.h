// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "SVGThumbnailRenderer.generated.h"

UCLASS()
class USVGThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	//~ Begin UThumbnailRenderer Interface.
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	//~ EndUThumbnailRenderer Interface.
};
