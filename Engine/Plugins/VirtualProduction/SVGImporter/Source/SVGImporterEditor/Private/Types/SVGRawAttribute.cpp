// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGRawAttribute.h"
#include "SVGImporterEditorModule.h"
#include "SVGParsingUtils.h"
#include "SVGPath.h"
#include "SVGRawElement.h"

TSharedRef<FSVGRawAttribute> FSVGRawAttribute::NewSVGAttribute(const TSharedPtr<FSVGRawElement>& InParentElement, const FName& InName, const FString& InValue)
{	
	return MakeShared<FSVGRawAttribute>(FSVGRawAttribute::FPrivateToken{}
		, InParentElement.ToSharedRef(), InName, InValue.TrimStartAndEnd());
}

void FSVGRawAttribute::PrintDebugInfo() const
{
	UE_LOGF(SVGImporterEditorLog, Log, "\t Attribute name: %ls, value: %ls", *Name.ToString(), *Value);
}

float FSVGRawAttribute::AsFloat() const
{
	return FSVGParsingUtils::FloatFromString(Value);
}

FString FSVGRawAttribute::AsString() const
{
	return Value;
}

void FSVGRawAttribute::AsPoints(TArray<FVector2D>& OutPoints) const
{
	FSVGParsingUtils::PointsFromString(Value,OutPoints);
}

bool FSVGRawAttribute::AsFloatWithSuffix(const FString& InSuffix, float& OutValue) const
{
	return FSVGParsingUtils::FloatFromStringWithSuffix(Value, InSuffix, OutValue);
}

TArray<TArray<FSVGPathCommand>> FSVGRawAttribute::AsPathCommands() const
{
	return FSVGParsingUtils::ParseStringAsPathCommands(Value);
}
