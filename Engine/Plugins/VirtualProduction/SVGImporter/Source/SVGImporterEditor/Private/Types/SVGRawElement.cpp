// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVGRawElement.h"
#include "SVGImporterEditorModule.h"
#include "SVGRawAttribute.h"

TSharedRef<FSVGRawElement> FSVGRawElement::NewSVGElement(const TSharedPtr<FSVGRawElement>& InParentElement, const FName& InName, const FString& InValue)
{
	return MakeShared<FSVGRawElement>(FSVGRawElement::FPrivateToken{}, InParentElement.ToSharedRef(), InName, InValue.TrimStartAndEnd());
}

void FSVGRawElement::AddChild(const TSharedRef<FSVGRawElement>& InChild)
{
	ChildrenElements.Emplace(InChild);
}

void FSVGRawElement::AddAttribute(const TSharedRef<FSVGRawAttribute>& InAttribute)
{
	AttributesMap.Add(InAttribute->Name, InAttribute);
}

void FSVGRawElement::PrintDebugInfo()
{
	UE_LOGF(SVGImporterEditorLog, Log, "Element name: %ls, value: %ls", *Name.ToString(), *Value);

	for (const TPair<FName, TSharedPtr<FSVGRawAttribute>>& Elem : AttributesMap)
	{
		Elem.Value->PrintDebugInfo();
	}
	
	for (const TSharedPtr<FSVGRawElement>& Elem : ChildrenElements)
	{
		Elem->PrintDebugInfo();
	}
}

bool FSVGRawElement::HasAttribute(const FName& InAttributeName) const
{
	return AttributesMap.Contains(InAttributeName);
}

TSharedPtr<FSVGRawAttribute> FSVGRawElement::GetAttribute(const FName& InAttributeName)
{
	return *AttributesMap.Find(InAttributeName);
}
