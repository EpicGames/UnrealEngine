// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SVGRawItem.h"
#include "Templates/SharedPointer.h"

struct FSVGRawAttribute;

/**
 * FSVGRawElement:
 * Elements, directly as parsed by the SVG parser (string values).
 * 
 * - They have references to their children elements.
 * - They can contain FSVGAttributes
 */
struct FSVGRawElement : public FSVGRawItem
{
public:
	/** Create a Raw Element Shared Ref */
	static TSharedRef<FSVGRawElement> NewSVGElement(const TSharedPtr<FSVGRawElement>& InParentElement, const FName& InName, const FString& InValue);

	/** Adds a child Element to this Element */
	void AddChild(const TSharedRef<FSVGRawElement>& InChild);

	/** Adds an Attribute to this Element */
	void AddAttribute(const TSharedRef<FSVGRawAttribute>& InAttribute);

	/** Prints information about this Element (own Attributes and Children) */
	void PrintDebugInfo();

	/** Checks if this Element has the specified Attribute */
	bool HasAttribute(const FName& InAttributeName) const;
	
	/** Retrieves the specified Attribute */
	TSharedPtr<FSVGRawAttribute> GetAttribute(const FName& InAttributeName);

	/** Get Children attached to this Element */
	const TArray<TSharedPtr<FSVGRawElement>>& GetChildren() const { return ChildrenElements; }

	/** Get Attributes of this Element*/
	const TMap<FName, TSharedPtr<FSVGRawAttribute>>& GetAttributes() const { return AttributesMap; }

private:
	/** A list of Children Elements*/
	TArray<TSharedPtr<FSVGRawElement>> ChildrenElements;

	/** A Map containing all the Attributes of this Element, indexed by name */
	TMap<FName, TSharedPtr<FSVGRawAttribute>> AttributesMap;
	
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FSVGRawElement(FPrivateToken, const TSharedRef<FSVGRawItem>& InParent, const FName& InName, const FString& InValue)
	{
		Parent = InParent;
		Name = InName;
		Value = InValue;
	}

	FSVGRawElement() = default;
};
