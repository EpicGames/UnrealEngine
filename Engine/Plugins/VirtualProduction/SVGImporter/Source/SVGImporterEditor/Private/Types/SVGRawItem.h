// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

/**
 * Base for non-converted SVG Elements and Attributes
 * SVGRawItems store the parsed information from the SVG as it is.
 * Therefore, they need to be converted in order to be used in engine.
 */
struct FSVGRawItem
{
public:

	FSVGRawItem()
	{}
	
	FSVGRawItem(const TSharedRef<FSVGRawItem>& InParent, const FName& InName, const FString& InValue)
		: Parent(InParent), Name(InName), Value(InValue)
	{}
	
	/** The parent of this Item */
	TSharedPtr<FSVGRawItem> Parent = nullptr;

	/** The name of this Item */
	FName Name = NAME_None;

	/** The value of this item, as taken from the SVG text buffer*/
	FString Value = TEXT("");
};
