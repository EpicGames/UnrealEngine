// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "SVGActorFactory.generated.h"

UCLASS(MinimalAPI)
class USVGActorFactory : public UActorFactory
{
	GENERATED_BODY()

	USVGActorFactory(const FObjectInitializer& ObjectInitializer);

protected:
	//~ Begin UActorFactory
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual AActor* GetDefaultActor(const FAssetData& AssetData) override;
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	//~ End UActorFactory
};
