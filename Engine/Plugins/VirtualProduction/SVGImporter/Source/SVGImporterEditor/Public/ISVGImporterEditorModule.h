// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class AActor;
class ULevel;
class UToolMenu;

class ISVGImporterEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("SVGImporterEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static ISVGImporterEditorModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<ISVGImporterEditorModule>(ModuleName);
	}

	virtual const FName GetStyleName() const = 0;

	virtual FName GetSVGImporterMenuCategoryName() const = 0;

	virtual void AddSVGActorMenuEntries(UToolMenu* InMenu, TSet<TWeakObjectPtr<AActor>> InActors) = 0;
};
