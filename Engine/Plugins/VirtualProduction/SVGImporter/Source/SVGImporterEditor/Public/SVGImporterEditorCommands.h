// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FSVGImporterEditorCommands : public TCommands<FSVGImporterEditorCommands>
{
public:
	SVGIMPORTEREDITOR_API static const FSVGImporterEditorCommands& GetExternal();

	static const FSVGImporterEditorCommands& GetInternal();

	FSVGImporterEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> SpawnSVGActor;

private:
	// Make this unavailable to the public
	using TCommands<FSVGImporterEditorCommands>::Get;
};
