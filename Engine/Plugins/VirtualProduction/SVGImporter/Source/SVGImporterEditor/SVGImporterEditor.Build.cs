// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class SVGImporterEditor : ModuleRules
{
	public SVGImporterEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Nanosvg");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetTools",
				"ContentBrowser",
				"CoreUObject",
				"DetailCustomizations",
				"EditorFramework",
				"Engine",
				"InputCore",
				"LevelEditor",
				"Projects",
				"PropertyEditor",
				"pugixml",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"SVGImporter",
				"ToolMenus",
				"TypedElementRuntime",
				"UnrealEd",
				"XmlParser",
			}
		);
	}
}
